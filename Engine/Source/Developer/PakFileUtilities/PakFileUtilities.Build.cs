// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PakFileUtilities : ModuleRules
{
	public PakFileUtilities(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		PrivateDependencyModuleNames.AddRange(new string[] {
			"Core",
			"PakFile",
			"Json",
			"Projects",
			"RSA",
			"IoStoreUtilities",
		});

		PrivateIncludePathModuleNames.AddRange(new string[] {
			"DerivedDataCache",
		});

		if (Target.bBuildWithEditorOnlyData)
		{
			DynamicallyLoadedModuleNames.AddRange(new string[] {
				"DerivedDataCache",
				"Virtualization",
			});
		}
	}
}
