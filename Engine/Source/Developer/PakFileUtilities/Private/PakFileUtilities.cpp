// Copyright Epic Games, Inc. All Rights Reserved.

#include "PakFileUtilities.h"
#include "IPlatformFilePak.h"
#include "Misc/SecureHash.h"
#include "Math/BigInt.h"
#include "SignedArchiveWriter.h"
#include "Misc/AES.h"
#include "Templates/UniquePtr.h"
#include "Serialization/LargeMemoryWriter.h"
#include "ProfilingDebugging/DiagnosticTable.h"
#include "Serialization/JsonSerializer.h"
#include "Misc/Base64.h"
#include "Misc/Compression.h"
#include "Misc/Fnv.h"
#include "Features/IModularFeatures.h"
#include "Misc/CoreDelegates.h"
#include "Misc/FileHelper.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "HAL/PlatformFileManager.h"
#include "Hash/xxhash.h"
#include "Async/ParallelFor.h"
#include "Async/AsyncWork.h"
#include "Modules/ModuleManager.h"
#include "DerivedDataCacheInterface.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/FileRegions.h"
#include "Misc/ICompressionFormat.h"
#include "Misc/KeyChainUtilities.h"
#include "IoStoreUtilities.h"
#include "Interfaces/IPluginManager.h"
#include "Containers/SpscQueue.h"
#include "Async/Async.h"
#include "Async/Future.h"
#include "Virtualization/VirtualizationSystem.h"
#include "CookedPackageStore.h"
#include "ZenStoreHttpClient.h"

IMPLEMENT_MODULE(FDefaultModuleImpl, PakFileUtilities);

#define GUARANTEE_UASSET_AND_UEXP_IN_SAME_PAK 0

#define USE_DDC_FOR_COMPRESSED_FILES 0
#define PAKCOMPRESS_DERIVEDDATA_VER TEXT("9493D2AB515048658AF7BE1342EC21FC")

DEFINE_LOG_CATEGORY_STATIC(LogMakeBinaryConfig, Log, All);


#define SEEK_OPT_VERBOSITY Display

#define DETAILED_UNREALPAK_TIMING 0
#if DETAILED_UNREALPAK_TIMING
struct FUnrealPakScopeCycleCounter
{
	volatile int64& Counter;
	uint32 StartTime;
	FUnrealPakScopeCycleCounter(volatile int64& InCounter) : Counter(InCounter) { StartTime = FPlatformTime::Cycles(); }
	~FUnrealPakScopeCycleCounter() 
	{
		uint32 EndTime = FPlatformTime::Cycles();
		volatile int64 DeltaTime = EndTime;
		if (EndTime > StartTime)
		{
			DeltaTime = EndTime - StartTime;
		}
		FPlatformAtomics::InterlockedAdd(&Counter, DeltaTime);
	}
};
volatile int64 GCompressionTime = 0;
volatile int64 GDDCSyncReadTime = 0;
volatile int64 GDDCSyncWriteTime = 0;
int64 GDDCHits = 0;
int64 GDDCMisses = 0;
#endif

bool ListFilesInPak(const TCHAR * InPakFilename, int64 SizeFilter, bool bIncludeDeleted, const FString& CSVFilename, bool bExtractToMountPoint, const FKeyChain& InKeyChain, bool bAppendFile, bool bIncludeFooter);

bool SignPakFile(const FString& InPakFilename, const FRSAKeyHandle InSigningKey)
{
	FString SignatureFilename(FPaths::ChangeExtension(InPakFilename, TEXT(".sig")));
	TUniquePtr<FArchive> PakFile(IFileManager::Get().CreateFileReader(*InPakFilename));

	const int64 TotalSize = PakFile->TotalSize();
	bool bFoundMagic = false;
	FPakInfo PakInfo;
	const int64 FileInfoSize = PakInfo.GetSerializedSize(FPakInfo::PakFile_Version_Latest);
	if (TotalSize >= FileInfoSize)
	{
		const int64 FileInfoPos = TotalSize - FileInfoSize;
		PakFile->Seek(FileInfoPos);
		PakInfo.Serialize(*PakFile.Get(), FPakInfo::PakFile_Version_Latest);
		bFoundMagic = (PakInfo.Magic == FPakInfo::PakFile_Magic);
	}

	// TODO: This should probably mimic the logic of FPakFile::Initialize which iterates back through pak versions until it find a compatible one
	if (!bFoundMagic)
	{
		return false;
	}

	TArray<uint8> ComputedSignatureData;
	ComputedSignatureData.Append(PakInfo.IndexHash.Hash, UE_ARRAY_COUNT(FSHAHash::Hash));

	const uint64 BlockSize = FPakInfo::MaxChunkDataSize;
	uint64 Remaining = PakFile->TotalSize();

	PakFile->Seek(0);

	TArray<uint8> Buffer;
	Buffer.SetNum(BlockSize);

	TArray<TPakChunkHash> Hashes;
	Hashes.Empty(IntCastChecked<int32>(PakFile->TotalSize() / BlockSize));

	while (Remaining > 0)
	{
		const uint64 CurrentBlockSize = FMath::Min(BlockSize, Remaining);
		Remaining -= CurrentBlockSize;

		PakFile->Serialize(Buffer.GetData(), CurrentBlockSize);
		Hashes.Add(ComputePakChunkHash(Buffer.GetData(), CurrentBlockSize));
	}

	FPakSignatureFile Signatures;
	Signatures.SetChunkHashesAndSign(Hashes, ComputedSignatureData, InSigningKey);

	TUniquePtr<FArchive> SignatureFile(IFileManager::Get().CreateFileWriter(*SignatureFilename));
	Signatures.Serialize(*SignatureFile.Get());

	const bool bSuccess = !PakFile->IsError() && !SignatureFile->IsError();
	return bSuccess;
}

bool SignIOStoreContainer(FArchive& ContainerAr)
{
	return true;
}


class FMemoryCompressor;

/**
* AsyncTask for FMemoryCompressor
* Compress a memory block asynchronously
 */
class FBlockCompressTask : public FNonAbandonableTask
{
public:
	friend class FAsyncTask<FBlockCompressTask>;
	friend class FMemoryCompressor;
	FBlockCompressTask(void* InUncompressedBuffer, int32 InUncompressedSize, FName InFormat, int32 InBlockSize, std::atomic<int32>& InRemainingTasksCounter, FGraphEventRef InAllTasksFinishedEvent) :
		RemainingTasksCounter(InRemainingTasksCounter),
		AllTasksFinishedEvent(InAllTasksFinishedEvent),
		UncompressedBuffer(InUncompressedBuffer),
		UncompressedSize(InUncompressedSize),
		Format(InFormat),
		BlockSize(InBlockSize),
		Result(false)
	{
		// Store buffer size.
		CompressedSize = FCompression::CompressMemoryBound(Format, BlockSize);
		CompressedBuffer = FMemory::Malloc(CompressedSize);
	}

	~FBlockCompressTask()
	{
		FMemory::Free(CompressedBuffer);
	}

	/** Do compress */
	void DoWork()
	{
#if DETAILED_UNREALPAK_TIMING
		FUnrealPakScopeCycleCounter Scope(GCompressionTime);
#endif
		// Compress memory block.
		// Actual size will be stored to CompressedSize.
		Result = FCompression::CompressMemory(Format, CompressedBuffer, CompressedSize, UncompressedBuffer, UncompressedSize, COMPRESS_ForPackaging);
		if (--RemainingTasksCounter == 0)
		{
			AllTasksFinishedEvent->DispatchSubsequents();
		}
	}

	
	FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(ExampleAsyncTask, STATGROUP_ThreadPoolAsyncTasks); }
		
private:
	std::atomic<int32>& RemainingTasksCounter;
	FGraphEventRef AllTasksFinishedEvent;

	// Source buffer
	void* UncompressedBuffer;
	int32 UncompressedSize;

	// Compress parameters
	FName Format;
	int32 BlockSize;
	int32 BitWindow;

	// Compressed result
	void* CompressedBuffer;
	int32 CompressedSize;
	bool Result;
};

/**
* asynchronous memory compressor
*/
class FMemoryCompressor
{
public:
	/** Divide into blocks and start compress asynchronously */
	FMemoryCompressor(uint8* UncompressedBuffer, int64 UncompressedSize, FName Format, int32 CompressionBlockSize, FGraphEventRef InCompressionFinishedEvent) 
		: CompressionFinishedEvent(InCompressionFinishedEvent)
		, Index(0)
	{
		// Divide into blocks and start compression async tasks.
		// These blocks must be as same as followed CompressMemory callings.
		int64 UncompressedBytes = 0;
		while (UncompressedSize)
		{
			int32 BlockSize = (int32)FMath::Min<int64>(UncompressedSize, CompressionBlockSize);
			auto* AsyncTask = new FAsyncTask<FBlockCompressTask>(UncompressedBuffer + UncompressedBytes, BlockSize, Format, BlockSize, RemainingTasksCounter, CompressionFinishedEvent);
			BlockCompressAsyncTasks.Add(AsyncTask);
			UncompressedSize -= BlockSize;
			UncompressedBytes += BlockSize;
		}	
	}

	~FMemoryCompressor()
	{
		for (FAsyncTask<FBlockCompressTask>* AsyncTask : BlockCompressAsyncTasks)
		{
			check(AsyncTask->IsDone());
			delete AsyncTask;
		}
	}

	void StartWork()
	{
		if (BlockCompressAsyncTasks.IsEmpty())
		{
			CompressionFinishedEvent->DispatchSubsequents();
		}
		else
		{
			int32 AsyncTasksCount = BlockCompressAsyncTasks.Num();
			RemainingTasksCounter = AsyncTasksCount;
			
			for (int32 TaskIndex = 0; TaskIndex < AsyncTasksCount; ++TaskIndex)
			{
				BlockCompressAsyncTasks[TaskIndex]->StartBackgroundTask();
			}
		}
	}

	/** Fetch compressed result. Returns true and store CompressedSize if succeeded */
	bool CopyNextCompressedBuffer(FName Format, void* CompressedBuffer, int32& CompressedSize, const void* UncompressedBuffer, int32 UncompressedSize)
	{
		// Fetch compressed result from task.
		// We assume this is called only once, same order, same parameters for
		// each task.
		FAsyncTask<FBlockCompressTask>* AsyncTask = BlockCompressAsyncTasks[Index++];
		while (!AsyncTask->IsDone())
		{
			// Compression is done but we also need to wait for the async task to be marked as completed before we call GetTask() below
			// DON'T call EnsureCompletion here, we're already running in a task so we don't want to pull in other tasks while waiting
			FPlatformProcess::Sleep(0);
		}
		FBlockCompressTask& Task = AsyncTask->GetTask();
		check(Task.Format == Format);
		check(Task.UncompressedBuffer == UncompressedBuffer);
		check(Task.UncompressedSize == UncompressedSize);
		check(CompressedSize >= Task.CompressedSize);
		if (!Task.Result)
		{
			return false;
		}
		FMemory::Memcpy(CompressedBuffer, Task.CompressedBuffer, Task.CompressedSize);
		CompressedSize = Task.CompressedSize;

		return true;
	}

private:
	FGraphEventRef CompressionFinishedEvent;

	TArray<FAsyncTask<FBlockCompressTask>*> BlockCompressAsyncTasks;
	std::atomic<int32> RemainingTasksCounter = 0;

	// Fetched task index
	int32 Index;
};

bool FPakOrderMap::ProcessOrderFile(const TCHAR* ResponseFile, bool bSecondaryOrderFile, bool bMergeOrder, TOptional<uint64> InOffset)
{
	uint64 OrderOffset = 0; 
	int32 OpenOrderNumber = 0;

	if (InOffset.IsSet())
	{
		OrderOffset = InOffset.GetValue();
	}
	else if (bSecondaryOrderFile || bMergeOrder)
	{
		OrderOffset = MaxIndex + 1;
	}

	if (bSecondaryOrderFile)
	{
		MaxPrimaryOrderIndex = OrderOffset;
	}
	// List of all items to add to pak file
	FString Text;
	UE_LOGF(LogPakFile, Display, "Loading pak order file %ls...", ResponseFile);
	if (FFileHelper::LoadFileToString(Text, ResponseFile))
	{
		// Read all lines
		TArray<FString> Lines;
		Text.ParseIntoArray(Lines, TEXT("\n"), true);
		for (int32 EntryIndex = 0; EntryIndex < Lines.Num(); EntryIndex++)
		{
			FString Path;
			Lines[EntryIndex].ReplaceInline(TEXT("\r"), TEXT(""));
			Lines[EntryIndex].ReplaceInline(TEXT("\n"), TEXT(""));
			const TCHAR* OrderLinePtr = *(Lines[EntryIndex]);
			// Skip comments
			if (FCString::Strncmp(OrderLinePtr, TEXT("#"), 1) == 0 || FCString::Strncmp(OrderLinePtr, TEXT("//"), 2) == 0)
			{
				continue;
			}

			if (!FParse::Token(OrderLinePtr, Path, false))
			{
				UE_LOGF(LogPakFile, Error, "Invalid entry in the response file %ls.", *Lines[EntryIndex]);
				return false;
			}

			if(Lines[EntryIndex].FindLastChar('"', OpenOrderNumber))
			{
				FString ReadNum = Lines[EntryIndex].RightChop(OpenOrderNumber + 1);
				Lines[EntryIndex].LeftInline(OpenOrderNumber + 1, EAllowShrinking::No);
				ReadNum.TrimStartInline();
				if(ReadNum.Len() == 0)
				{
					// If order files don't have explicit numbers just use the line number
					OpenOrderNumber = EntryIndex;
				}
				else if (ReadNum.IsNumeric())
				{
					OpenOrderNumber = FCString::Atoi(*ReadNum);
				}
				else
				{
					UE_LOGF(LogPakFile, Error, "Invalid entry in the response file %ls, couldn't parse an order number after the path.", *Lines[EntryIndex]);
					return false;
				}
			}
			else
			{
				// If order files don't have explicit numbers just use the line number
				OpenOrderNumber = EntryIndex;
			}

			FPaths::NormalizeFilename(Path);
			Path = Path.ToLower();

			if ((bSecondaryOrderFile || bMergeOrder) && OrderMap.Contains(Path))
			{
				continue;
			}

			OrderMap.Add(Path, OpenOrderNumber + OrderOffset);
			MaxIndex = FMath::Max(MaxIndex, OpenOrderNumber + OrderOffset);
		}

		UE_LOGF(LogPakFile, Display, "Finished loading pak order file %ls.", ResponseFile);
		return true;
	}
	else
	{
		UE_LOGF(LogPakFile, Error, "Unable to load pak order file %ls.", ResponseFile);
		return false;
	}
}

void FPakOrderMap::MergeOrderMap(FPakOrderMap&& Other)
{
	for (TPair<FString, uint64>& OrderedFile : Other.OrderMap)
	{
		if (OrderMap.Contains(OrderedFile.Key) == false)
		{
			OrderMap.Add(MoveTemp(OrderedFile.Key), OrderedFile.Value);
		}
	}
	Other.OrderMap.Empty(); // We moved strings out of this so empty it so nobody tries to use the keys

	if (MaxIndex == MAX_uint64)
	{
		MaxIndex = Other.MaxIndex;
	}
	else
	{
		MaxIndex = FMath::Max(MaxIndex, Other.MaxIndex);
	}
}

uint64 FPakOrderMap::GetFileOrder(const FString& Path, bool bAllowUexpUBulkFallback, bool* OutIsPrimary) const
{
	FString RegionStr;
	FString NewPath = RemapLocalizationPathIfNeeded(Path.ToLower(), RegionStr);
	const uint64* FoundOrder = OrderMap.Find(NewPath);
	uint64 ReturnOrder = MAX_uint64;
	if (FoundOrder != nullptr)
	{
		ReturnOrder = *FoundOrder;
		if (OutIsPrimary)
		{
			*OutIsPrimary = (ReturnOrder < MaxPrimaryOrderIndex);
		}
	}
	else if (bAllowUexpUBulkFallback)
	{
		// if this is a cook order or an old order it will not have uexp files in it, so we put those in the same relative order after all of the normal files, but before any ubulk files
		if (Path.EndsWith(TEXT("uexp")) || Path.EndsWith(TEXT("ubulk")))
		{
			uint64 CounterpartOrder = GetFileOrder(FPaths::GetBaseFilename(Path, false) + TEXT(".uasset"), false);
			if (CounterpartOrder == MAX_uint64)
			{
				CounterpartOrder = GetFileOrder(FPaths::GetBaseFilename(Path, false) + TEXT(".umap"), false);
			}
			if (CounterpartOrder != MAX_uint64)
			{
				if (Path.EndsWith(TEXT("uexp")))
				{
					ReturnOrder = CounterpartOrder | (1 << 29);
				}
				else
				{
					ReturnOrder = CounterpartOrder | (1 << 30);
				}
			}
		}
	}

	// Optionally offset based on region, so multiple files in different regions don't get the same order.
	// I/O profiling suggests this is slightly worse, so leaving this disabled for now
#if 0
	if (ReturnOrder != MAX_uint64)
	{
		if (RegionStr.Len() > 0)
		{
			uint64 RegionOffset = 0;
			for (int i = 0; i < RegionStr.Len(); i++)
			{
				int8 Letter = (int8)(RegionStr[i] - TEXT('a'));
				RegionOffset |= (uint64(Letter) << (i * 5));
			}
			return ReturnOrder + (RegionOffset << 16);
		}
	}
#endif
	return ReturnOrder;
}

void FPakOrderMap::WriteOpenOrder(FArchive* Ar)
{
	OrderMap.ValueSort([](const uint64& A, const uint64& B) { return A < B; });
	for (const auto& It : OrderMap)
	{
		Ar->Logf(TEXT("\"%s\" %d"), *It.Key, It.Value);
	}
}

FString FPakOrderMap::RemapLocalizationPathIfNeeded(const FString& PathLower, FString& OutRegion) const
{
	static const TCHAR* L10NPrefix = (const TCHAR*)TEXT("/content/l10n/");
	static const int32 L10NPrefixLength = FCString::Strlen(L10NPrefix);
	int32 FoundIndex = PathLower.Find(L10NPrefix, ESearchCase::CaseSensitive);
	if (FoundIndex > 0)
	{
		// Validate the content index is the first one
		int32 ContentIndex = PathLower.Find(TEXT("/content/"), ESearchCase::CaseSensitive);
		if (ContentIndex == FoundIndex)
		{
			int32 EndL10NOffset = ContentIndex + L10NPrefixLength;
			int32 NextSlashIndex = PathLower.Find(TEXT("/"), ESearchCase::CaseSensitive, ESearchDir::FromStart, EndL10NOffset);
			int32 RegionLength = NextSlashIndex - EndL10NOffset;
			if (RegionLength >= 2)
			{
				FString NonLocalizedPath = PathLower.Mid(0, ContentIndex) + TEXT("/content") + PathLower.Mid(NextSlashIndex);
				OutRegion = PathLower.Mid(EndL10NOffset, RegionLength);
				return NonLocalizedPath;
			}
		}
	}
	return PathLower;
}

enum class ESeekOptMode : uint8
{
	None = 0,
	OnePass = 1,
	Incremental = 2,
	Incremental_OnlyPrimaryOrder = 3,
	Incremental_PrimaryThenSecondary = 4,
	COUNT
};

struct FPatchSeekOptParams
{
	FPatchSeekOptParams()
		: MaxGapSize(0)
		, MaxInflationPercent(0.0f)
		, Mode(ESeekOptMode::None)
		, MaxAdjacentOrderDiff(128)
	{}
	int64 MaxGapSize;
	float MaxInflationPercent; // For Incremental_ modes only
	ESeekOptMode Mode;
	int32 MaxAdjacentOrderDiff;
};

struct FPakCommandLineParameters
{
	FPakCommandLineParameters()
		: CompressionBlockSize(64 * 1024)
		, FileSystemBlockSize(0)
		, PatchFilePadAlign(0)
		, AlignForMemoryMapping(0)
		, GeneratePatch(false)
		, EncryptIndex(false)
		, UseCustomCompressor(false)
		, bSign(false)
		, bPatchCompatibilityMode421(false)
		, bFallbackOrderForNonUassetFiles(false)
		, bAlignFilesLargerThanBlock(false)
		, bForceCompress(false)
		, bFileRegions(false)
		, bRequiresRehydration(false)
	{
	}

	TArray<FName> CompressionFormats;
	FPatchSeekOptParams SeekOptParams;
	int32  CompressionBlockSize;
	int64  FileSystemBlockSize;
	int64  PatchFilePadAlign;
	int64  AlignForMemoryMapping;
	bool   GeneratePatch;
	FString SourcePatchPakFilename;
	FString SourcePatchDiffDirectory;
	FString InputFinalPakFilename; // This is the resulting pak file we want to end up with after we generate the pak patch.  This is used instead of passing in the raw content.
	FString ChangedFilesOutputFilename;
	FString CsvPath;
	FString ProjectStoreFilename;
	bool EncryptIndex;
	bool UseCustomCompressor;
	FGuid EncryptionKeyGuid;
	bool bSign;
	bool bPatchCompatibilityMode421;
	bool bFallbackOrderForNonUassetFiles;
	bool bAlignFilesLargerThanBlock;	// Align files that are larger than block size

	bool bForceCompress; // Force all files that request compression to be compressed, even if that results in a larger file size DEPRECATED
	bool bFileRegions; // Enables the processing and output of cook file region metadata, used during packaging on some platforms.

	bool bRequiresRehydration; // One or more files to be added to the pak file are virtualized and require rehydration.
};

struct FPakInputPair
{
	FString Source;
	FString Dest;
	uint64 SuggestedOrder; 
	bool bNeedsCompression;
	bool bNeedEncryption;
	bool bIsDeleteRecord;	// This is used for patch PAKs when a file is deleted from one patch to the next
	bool bIsInPrimaryOrder;
	bool bNeedRehydration;

	FPakInputPair()
		: SuggestedOrder(MAX_uint64)
		, bNeedsCompression(false)
		, bNeedEncryption(false)
		, bIsDeleteRecord(false)
		, bIsInPrimaryOrder(false)
		, bNeedRehydration(false)
	{}

	FPakInputPair(const FString& InSource, const FString& InDest)
		: Source(InSource)
		, Dest(InDest)
		, bNeedsCompression(false)
		, bNeedEncryption(false)
		, bIsDeleteRecord(false)
		, bNeedRehydration(false)
	{}

	FORCEINLINE bool operator==(const FPakInputPair& Other) const
	{
		return Source == Other.Source;
	}
};

struct FPakEntryOrder
{
	FPakEntryOrder() : Order(MAX_uint64) {}
	FString Filename;
	uint64  Order;
};


struct FFileInfo
{
	uint64 FileSize;
	int32 PatchIndex;
	bool bIsDeleteRecord;
	bool bForceInclude;
	uint8 Hash[16];
};

bool ExtractFilesFromPak(const TCHAR* InPakFilename, TMap<FString, FFileInfo>& InFileHashes, 
	const TCHAR* InDestPath, bool bUseMountPoint, 
	const FKeyChain& InKeyChain, const FString* InFilter, 
	TArray<FPakInputPair>* OutEntries = nullptr, TArray<FPakInputPair>* OutDeletedEntries = nullptr, 
	FPakOrderMap* OutOrderMap = nullptr, TArray<FGuid>* OutUsedEncryptionKeys = nullptr, bool* OutAnyPakSigned = nullptr);

struct FCompressedFileBuffer
{
	FCompressedFileBuffer()
		: OriginalSize(0)
		, TotalCompressedSize(0)
		, FileCompressionBlockSize(0)
		, CompressedBufferSize(0)
		, RehydrationCount(0)
		, RehydrationBytes(0)
	{

	}

	void Reinitialize(FName CompressionMethod, int64 CompressionBlockSize)
	{
		TotalCompressedSize = 0;
		FileCompressionBlockSize = 0;
		FileCompressionMethod = CompressionMethod;
		CompressedBlocks.Reset();
		CompressedBlocks.AddUninitialized(IntCastChecked<int32>((OriginalSize+CompressionBlockSize-1)/CompressionBlockSize));
	}

	void Empty()
	{
		OriginalSize = 0;
		TotalCompressedSize = 0;
		FileCompressionBlockSize = 0;
		FileCompressionMethod = NAME_None;
		CompressedBuffer = nullptr;
		CompressedBufferSize = 0; 
		CompressedBlocks.Empty();
		RehydrationCount = 0;
		RehydrationBytes = 0;
	}

	void EnsureBufferSpace(int64 RequiredSpace)
	{
		if(RequiredSpace > CompressedBufferSize)
		{
			TUniquePtr<uint8[]> NewCompressedBuffer = MakeUnique<uint8[]>(RequiredSpace);
			FMemory::Memcpy(NewCompressedBuffer.Get(), CompressedBuffer.Get(), CompressedBufferSize);
			CompressedBuffer = MoveTemp(NewCompressedBuffer);
			CompressedBufferSize = RequiredSpace;
		}
	}

	void ResetSource();
	bool ReadSource(const FPakInputPair& InFile, FCookedPackageStore* PackageStore);
	void SetSourceAsWorkingBuffer();

	TUniquePtr<FMemoryCompressor> BeginCompressFileToWorkingBuffer(const FPakInputPair& InFile, FName CompressionMethod, const int32 CompressionBlockSize, FGraphEventRef EndCompressionBarrier);
	bool EndCompressFileToWorkingBuffer(const FPakInputPair& InFile, FName CompressionMethod, const int32 CompressionBlockSize, FMemoryCompressor& MemoryCompressor);

	void SerializeDDCData(FArchive &Ar)
	{
		Ar << OriginalSize;
		Ar << TotalCompressedSize;
		Ar << FileCompressionBlockSize;
		Ar << FileCompressionMethod;
		Ar << CompressedBlocks;
		if (Ar.IsLoading())
		{
			EnsureBufferSpace(TotalCompressedSize);
		}
		Ar.Serialize(CompressedBuffer.Get(), TotalCompressedSize);
	}

	int64 GetSerializedSizeEstimate() const
	{
		int64 Size = 0;
		Size += sizeof(*this);
		Size += CompressedBlocks.Num() * sizeof(FPakCompressedBlock);
		Size += CompressedBufferSize;
		return Size;
	}

	FString GetDDCKeyString(const uint8* UncompressedFile, const int64& UncompressedFileSize, FName CompressionFormat, const int64& BlockSize);

	/** Returns the size of the file, excluding any padding that might have been applied */
	int64 GetFileSize() const
	{
		return OriginalSize;
	}

	TArray64<uint8>		UncompressedBuffer;


	int64				OriginalSize;
	int64				TotalCompressedSize;
	int32				FileCompressionBlockSize;
	FName				FileCompressionMethod;
	TArray<FPakCompressedBlock>	CompressedBlocks;
	int64				CompressedBufferSize;
	TUniquePtr<uint8[]>		CompressedBuffer;
	int32				RehydrationCount;
	int64				RehydrationBytes;
};

template <class T>
bool ReadSizeParam(const TCHAR* CmdLine, const TCHAR* ParamStr, T& SizeOut)
{
	FString ParamValueStr;
	if (FParse::Value(CmdLine, ParamStr, ParamValueStr) &&
		FParse::Value(CmdLine, ParamStr, SizeOut))
	{
		if (ParamValueStr.EndsWith(TEXT("GB")))
		{
			SizeOut *= 1024 * 1024 * 1024;
		}
		else if (ParamValueStr.EndsWith(TEXT("MB")))
		{
			SizeOut *= 1024 * 1024;
		}
		else if (ParamValueStr.EndsWith(TEXT("KB")))
		{
			SizeOut *= 1024;
		}
		return true;
	}
	return false;
}


FString GetLongestPath(const TArray<FPakInputPair>& FilesToAdd)
{
	FString LongestPath;
	int32 MaxNumDirectories = 0;

	for (int32 FileIndex = 0; FileIndex < FilesToAdd.Num(); FileIndex++)
	{
		const FString& Filename = FilesToAdd[FileIndex].Dest;
		int32 NumDirectories = 0;
		for (int32 Index = 0; Index < Filename.Len(); Index++)
		{
			if (Filename[Index] == '/')
			{
				NumDirectories++;
			}
		}
		if (NumDirectories > MaxNumDirectories)
		{
			LongestPath = Filename;
			MaxNumDirectories = NumDirectories;
		}
	}
	return FPaths::GetPath(LongestPath) + TEXT("/");
}

FString GetCommonRootPath(const TArray<FPakInputPair>& FilesToAdd)
{
	FString Root = GetLongestPath(FilesToAdd);
	for (int32 FileIndex = 0; FileIndex < FilesToAdd.Num() && Root.Len(); FileIndex++)
	{
		FString Filename(FilesToAdd[FileIndex].Dest);
		FString Path = FPaths::GetPath(Filename) + TEXT("/");
		int32 CommonSeparatorIndex = -1;
		int32 SeparatorIndex = Path.Find(TEXT("/"), ESearchCase::CaseSensitive);
		while (SeparatorIndex >= 0)
		{
			if (FCString::Strnicmp(*Root, *Path, SeparatorIndex + 1) != 0)
			{
				break;
			}
			CommonSeparatorIndex = SeparatorIndex;
			if (CommonSeparatorIndex + 1 < Path.Len())
			{
				SeparatorIndex = Path.Find(TEXT("/"), ESearchCase::CaseSensitive, ESearchDir::FromStart, CommonSeparatorIndex + 1);
			}
			else
			{
				break;
			}
		}
		if ((CommonSeparatorIndex + 1) < Root.Len())
		{
			Root.MidInline(0, CommonSeparatorIndex + 1, EAllowShrinking::No);
		}
	}
	return Root;
}


FString FCompressedFileBuffer::GetDDCKeyString(const uint8* UncompressedFile, const int64& UncompressedFileSize, FName CompressionFormat, const int64& BlockSize)
{
	FString KeyString;

	KeyString += FString::Printf(TEXT("_F:%s_C:%s_B:%" INT64_FMT "_"), *CompressionFormat.ToString(), *FCompression::GetCompressorDDCSuffix(CompressionFormat), BlockSize);
	
	FSHA1 HashState;
	HashState.Update(UncompressedFile, UncompressedFileSize);
	HashState.Final();
	FSHAHash FinalHash;
	HashState.GetHash(FinalHash.Hash);
	KeyString += FinalHash.ToString();;

	return FDerivedDataCacheInterface::BuildCacheKey(TEXT("PAKCOMPRESS_"), PAKCOMPRESS_DERIVEDDATA_VER, *KeyString);
}

void FCompressedFileBuffer::ResetSource()
{
	UncompressedBuffer.Empty();
}

bool FCompressedFileBuffer::ReadSource(const FPakInputPair& InputInfo, FCookedPackageStore* PackageStore)
{
	if (!InputInfo.bNeedRehydration)
	{
		TUniquePtr<FArchive> FileHandle(IFileManager::Get().CreateFileReader(*InputInfo.Source));
		if (FileHandle)
		{
			OriginalSize = FileHandle->TotalSize();
			const int64 PaddedEncryptedFileSize = Align(OriginalSize, FAES::AESBlockSize);

			UncompressedBuffer.SetNumUninitialized(PaddedEncryptedFileSize);

			FileHandle->Serialize(UncompressedBuffer.GetData(), OriginalSize);

			if (!FileHandle->IsError())
			{
				return true;
			}
		}
		else if (PackageStore && PackageStore->HasZenStoreClient())
		{
			FString FullFilename = FPaths::ConvertRelativePathToFull(InputInfo.Source);
			FIoChunkId ChunkId = PackageStore->GetChunkIdFromFileName(FullFilename);
			if (ChunkId.IsValid())
			{
				TIoStatusOr<FIoBuffer> ChunkReadStatus = PackageStore->ReadChunk(ChunkId);
				if (ChunkReadStatus.IsOk())
				{
					FIoBuffer Buffer = ChunkReadStatus.ConsumeValueOrDie();
					OriginalSize = Buffer.GetSize();
					const int64 PaddedEncryptedFileSize = Align(OriginalSize, FAES::AESBlockSize);
					UncompressedBuffer.SetNumUninitialized(PaddedEncryptedFileSize);
					FMemory::Memcpy(UncompressedBuffer.GetData(), Buffer.GetData(), Buffer.GetSize());
					return true;
				}
			}
		}
		UncompressedBuffer.Empty();
		OriginalSize = -1;
		return false;
	}
	else
	{
		using namespace UE::Virtualization;
		IVirtualizationSystem& System = IVirtualizationSystem::Get();

		TArray<FString> PackagePath;
		PackagePath.Add(InputInfo.Source);

		TArray<FSharedBuffer> RehydratedPackage;
		TArray<FRehydrationInfo> RehydrationInfo;
		TArray<FText> Errors;

		if (System.TryRehydratePackages(PackagePath, FAES::AESBlockSize, Errors, RehydratedPackage, &RehydrationInfo) == ERehydrationResult::Success)
		{
			OriginalSize = RehydrationInfo[0].RehydratedSize;
			const int64 PaddedEncryptedFileSize = Align(OriginalSize, FAES::AESBlockSize);

			// Confirm padding worked
			check(RehydratedPackage[0].GetSize() == PaddedEncryptedFileSize);

			//TODO: Keep data in FSharedBuffer form, rather than TArray
			UncompressedBuffer.SetNumUninitialized(RehydratedPackage[0].GetSize());
			FMemory::Memcpy(UncompressedBuffer.GetData(), RehydratedPackage[0].GetData(), RehydratedPackage[0].GetSize());

			RehydrationCount = RehydrationInfo[0].NumPayloadsRehydrated;
			RehydrationBytes = RehydrationInfo[0].RehydratedSize - RehydrationInfo[0].OriginalSize;

			if (RehydrationCount)
			{
				UE_LOGF(LogPakFile, Verbose, "Rehydrated (+%lld bytes) %ls", RehydrationBytes, *PackagePath[0]);
			}

			return true;
		}
		else
		{
			UncompressedBuffer.Empty();
			OriginalSize = -1;

			return false;
		}
	}
}

void FCompressedFileBuffer::SetSourceAsWorkingBuffer()
{
	// TODO: Do not submit with this assert
	check(!UncompressedBuffer.IsEmpty() || OriginalSize == 0);

	// TODO: Better way to move the buffers ownership, but can't safely steal the memory in a
	// TArray
	CompressedBufferSize = UncompressedBuffer.Num();
	CompressedBuffer = MakeUnique<uint8[]>(CompressedBufferSize);

	FMemory::Memcpy(CompressedBuffer.Get(), UncompressedBuffer.GetData(), CompressedBufferSize);
	
	UncompressedBuffer.Empty();
}

TUniquePtr<FMemoryCompressor> FCompressedFileBuffer::BeginCompressFileToWorkingBuffer(const FPakInputPair& InFile, FName CompressionMethod, const int32 CompressionBlockSize, FGraphEventRef EndCompressionBarrier)
{
	check(!UncompressedBuffer.IsEmpty() || OriginalSize == 0);

	Reinitialize(CompressionMethod, CompressionBlockSize);

#if USE_DDC_FOR_COMPRESSED_FILES
	const bool bShouldUseDDC = true; // && (FileSize > 20 * 1024 ? true : false);
	FString DDCKey;
	TArray<uint8, TInlineAllocator<4096>> GetData;
	if (bShouldUseDDC)
	{
#if DETAILED_UNREALPAK_TIMING
		FUnrealPakScopeCycleCounter Scope(GDDCSyncReadTime);
#endif
		DDCKey = GetDDCKeyString(UncompressedBuffer.GetData(), FileSize, CompressionMethod, CompressionBlockSize);

		if (GetDerivedDataCacheRef().CachedDataProbablyExists(*DDCKey))
		{
			int32 AsyncHandle = GetDerivedDataCacheRef().GetAsynchronous(*DDCKey, InFile.Dest);
			GetDerivedDataCacheRef().WaitAsynchronousCompletion(AsyncHandle);
			GetData.Empty(GetData.Max());
			bool Result = false;
			GetDerivedDataCacheRef().GetAsynchronousResults(AsyncHandle, GetData, &Result);
			if (Result)
			{
				FMemoryReader Ar(GetData, true);
				SerializeDDCData(Ar);
				UncompressedBuffer.Empty();
				EndCompressionBarrier->DispatchSubsequents();
				return nullptr;
			}
		}
	}
#endif

	// Start parallel compress
	return MakeUnique<FMemoryCompressor>(UncompressedBuffer.GetData(), OriginalSize, CompressionMethod, CompressionBlockSize, EndCompressionBarrier);
}

bool FCompressedFileBuffer::EndCompressFileToWorkingBuffer(const FPakInputPair& InFile, FName CompressionMethod, const int32 CompressionBlockSize, FMemoryCompressor& MemoryCompressor)
{
	{
		// Build buffers for working
		int64 UncompressedSize = OriginalSize;

		// CompressMemoryBound truncates its size argument to 32bits, so we can not use (possibly > 32-bit) FileSize directly to calculate required buffer space
		int32 MaxCompressedBufferSize = Align(FCompression::CompressMemoryBound(CompressionMethod, CompressionBlockSize, COMPRESS_NoFlags), FAES::AESBlockSize);
		int32 CompressionBufferRemainder = Align(FCompression::CompressMemoryBound(CompressionMethod, int32(UncompressedSize % CompressionBlockSize), COMPRESS_NoFlags), FAES::AESBlockSize);
		EnsureBufferSpace(MaxCompressedBufferSize * (UncompressedSize / CompressionBlockSize) + CompressionBufferRemainder);

		TotalCompressedSize = 0;
		int64 UncompressedBytes = 0;
		int32 CurrentBlock = 0;
		while (UncompressedSize)
		{
			int32 BlockSize = (int32)FMath::Min<int64>(UncompressedSize, CompressionBlockSize);
			int32 MaxCompressedBlockSize = FCompression::CompressMemoryBound(CompressionMethod, BlockSize, COMPRESS_NoFlags);
			int32 CompressedBlockSize = FMath::Max<int32>(MaxCompressedBufferSize, MaxCompressedBlockSize);
			FileCompressionBlockSize = FMath::Max<uint32>(BlockSize, FileCompressionBlockSize);
			EnsureBufferSpace(Align(TotalCompressedSize + CompressedBlockSize, FAES::AESBlockSize));
			if (!MemoryCompressor.CopyNextCompressedBuffer(CompressionMethod, CompressedBuffer.Get() + TotalCompressedSize, CompressedBlockSize, UncompressedBuffer.GetData() + UncompressedBytes, BlockSize))
			{
				return false;
			}
			UncompressedSize -= BlockSize;
			UncompressedBytes += BlockSize;

			CompressedBlocks[CurrentBlock].CompressedStart = TotalCompressedSize;
			CompressedBlocks[CurrentBlock].CompressedEnd = TotalCompressedSize + CompressedBlockSize;
			++CurrentBlock;

			TotalCompressedSize += CompressedBlockSize;

			if (InFile.bNeedEncryption)
			{
				int64 EncryptionBlockPadding = Align(TotalCompressedSize, FAES::AESBlockSize);
				for (int64 FillIndex = TotalCompressedSize; FillIndex < EncryptionBlockPadding; ++FillIndex)
				{
					// Fill the trailing buffer with bytes from file. Note that this is now from a fixed location
					// rather than a random one so that we produce deterministic results
					CompressedBuffer.Get()[FillIndex] = CompressedBuffer.Get()[FillIndex % TotalCompressedSize];
				}
				TotalCompressedSize += EncryptionBlockPadding - TotalCompressedSize;
			}
		}

	}
#if USE_DDC_FOR_COMPRESSED_FILES
	if (bShouldUseDDC)
	{
#if DETAILED_UNREALPAK_TIMING
		FUnrealPakScopeCycleCounter Scope(GDDCSyncWriteTime);
		++GDDCMisses;
#endif
		GetData.Empty(GetData.Max());
		FMemoryWriter Ar(GetData, true);
		SerializeDDCData(Ar);
		GetDerivedDataCacheRef().Put(*DDCKey, GetData, InFile.Dest);
	}
#endif

	return true;
}

bool PrepareCopyFileToPak(const FString& InMountPoint, const FPakInputPair& InFile, const FCompressedFileBuffer& UncompressedFile, FPakEntryPair& OutNewEntry, uint8*& OutDataToWrite, int64& OutSizeToWrite, const FKeyChain& InKeyChain, TArray<FFileRegion>* OutFileRegions)
{
	const int64 FileSize = UncompressedFile.OriginalSize;
	const int64 PaddedEncryptedFileSize = Align(FileSize, FAES::AESBlockSize); 
	if (FileSize < 0)
	{
		return false;
	}
	check(UncompressedFile.CompressedBufferSize == PaddedEncryptedFileSize);

	OutNewEntry.Filename = InFile.Dest.Mid(InMountPoint.Len());
	OutNewEntry.Info.Offset = 0; // Don't serialize offsets here.
	OutNewEntry.Info.Size = FileSize;
	OutNewEntry.Info.UncompressedSize = FileSize;
	OutNewEntry.Info.CompressionMethodIndex = 0;
	OutNewEntry.Info.SetEncrypted( InFile.bNeedEncryption );
	OutNewEntry.Info.SetDeleteRecord(false);

	{
		OutSizeToWrite = FileSize;
		uint8* UncompressedBuffer = UncompressedFile.CompressedBuffer.Get();
		if (InFile.bNeedEncryption)
		{
			if (FileSize > 0)
			{
				for (int64 FillIndex = FileSize; FillIndex < PaddedEncryptedFileSize; ++FillIndex)
				{
					// Fill the trailing buffer with bytes from file. Note that this is now from a fixed location
					// rather than a random one so that we produce deterministic results
					UncompressedBuffer[FillIndex] = UncompressedBuffer[(FillIndex - FileSize) % FileSize];
				}
			}

			//Encrypt the buffer before writing it to disk
			check(InKeyChain.GetPrincipalEncryptionKey());
			FAES::EncryptData(UncompressedBuffer, PaddedEncryptedFileSize, InKeyChain.GetPrincipalEncryptionKey()->Key);
			// Update the size to be written
			OutSizeToWrite = PaddedEncryptedFileSize;
			OutNewEntry.Info.SetEncrypted( true );
		}

		// Calculate the buffer hash value
		FSHA1::HashBuffer(UncompressedBuffer, FileSize, OutNewEntry.Info.Hash);
		OutDataToWrite = UncompressedBuffer;
	}

	if (OutFileRegions)
	{
		// Read the matching regions file, if it exists.
		TUniquePtr<FArchive> RegionsFile(IFileManager::Get().CreateFileReader(*(InFile.Source + FFileRegion::RegionsFileExtension)));
		if (RegionsFile.IsValid())
		{
			FFileRegion::SerializeFileRegions(*RegionsFile.Get(), *OutFileRegions);
		}
	}
	return true;
}

void FinalizeCopyCompressedFileToPak(FPakInfo& InPakInfo, const FCompressedFileBuffer& CompressedFile, FPakEntryPair& OutNewEntry)
{
	check(CompressedFile.TotalCompressedSize != 0);

	check(OutNewEntry.Info.CompressionBlocks.Num() == CompressedFile.CompressedBlocks.Num());
	check(OutNewEntry.Info.CompressionMethodIndex == InPakInfo.GetCompressionMethodIndex(CompressedFile.FileCompressionMethod));

	int64 TellPos = OutNewEntry.Info.GetSerializedSize(FPakInfo::PakFile_Version_Latest);
	const TArray<FPakCompressedBlock>& Blocks = CompressedFile.CompressedBlocks;
	for (int32 BlockIndex = 0, BlockCount = Blocks.Num(); BlockIndex < BlockCount; ++BlockIndex)
	{
		OutNewEntry.Info.CompressionBlocks[BlockIndex].CompressedStart = Blocks[BlockIndex].CompressedStart + TellPos;
		OutNewEntry.Info.CompressionBlocks[BlockIndex].CompressedEnd = Blocks[BlockIndex].CompressedEnd + TellPos;
	}
}

bool PrepareCopyCompressedFileToPak(const FString& InMountPoint, FPakInfo& Info, const FPakInputPair& InFile, const FCompressedFileBuffer& CompressedFile, FPakEntryPair& OutNewEntry, uint8*& OutDataToWrite, int64& OutSizeToWrite, const FKeyChain& InKeyChain)
{
	if (CompressedFile.TotalCompressedSize == 0)
	{
		return false;
	}

	OutNewEntry.Info.CompressionMethodIndex = Info.GetCompressionMethodIndex(CompressedFile.FileCompressionMethod);
	OutNewEntry.Info.CompressionBlocks.AddZeroed(CompressedFile.CompressedBlocks.Num());

	if (InFile.bNeedEncryption)
	{
		check(InKeyChain.GetPrincipalEncryptionKey());
		FAES::EncryptData(CompressedFile.CompressedBuffer.Get(), CompressedFile.TotalCompressedSize, InKeyChain.GetPrincipalEncryptionKey()->Key);
	}

	//Hash the final buffer thats written
	FSHA1 Hash;
	Hash.Update(CompressedFile.CompressedBuffer.Get(), CompressedFile.TotalCompressedSize);
	Hash.Final();

	// Update file size & Hash
	OutNewEntry.Info.CompressionBlockSize = CompressedFile.FileCompressionBlockSize;
	OutNewEntry.Info.UncompressedSize = CompressedFile.OriginalSize;
	OutNewEntry.Info.Size = CompressedFile.TotalCompressedSize;
	Hash.GetHash(OutNewEntry.Info.Hash);

	//	Write the header, then the data
	OutNewEntry.Filename = InFile.Dest.Mid(InMountPoint.Len());
	OutNewEntry.Info.Offset = 0; // Don't serialize offsets here.
	OutNewEntry.Info.SetEncrypted( InFile.bNeedEncryption );
	OutNewEntry.Info.SetDeleteRecord(false);
	OutSizeToWrite = CompressedFile.TotalCompressedSize;
	OutDataToWrite = CompressedFile.CompressedBuffer.Get();
	//OutNewEntry.Info.Serialize(InPak,FPakInfo::PakFile_Version_Latest);	
	//InPak.Serialize(CompressedFile.CompressedBuffer.Get(), CompressedFile.TotalCompressedSize);

	return true;
}

void PrepareDeleteRecordForPak(const FString& InMountPoint, const FPakInputPair InDeletedFile, FPakEntryPair& OutNewEntry)
{
	OutNewEntry.Filename = InDeletedFile.Dest.Mid(InMountPoint.Len());
	OutNewEntry.Info.SetDeleteRecord(true);
}

void ProcessCommonCommandLine(const TCHAR* CmdLine, FPakCommandLineParameters& CmdLineParameters)
{
	// List of all items to add to pak file
	FString ClusterSizeString;

	if (FParse::Param(CmdLine, TEXT("patchcompatibilitymode421")))
	{
		CmdLineParameters.bPatchCompatibilityMode421 = true;
	}

	if (FParse::Param(CmdLine, TEXT("fallbackOrderForNonUassetFiles")))
	{
		CmdLineParameters.bFallbackOrderForNonUassetFiles = true;
	}

	if (FParse::Value(CmdLine, TEXT("-blocksize="), ClusterSizeString) && 
		FParse::Value(CmdLine, TEXT("-blocksize="), CmdLineParameters.FileSystemBlockSize))
		{
		if (ClusterSizeString.EndsWith(TEXT("MB")))
		{
			CmdLineParameters.FileSystemBlockSize *= 1024*1024;
		}
		else if (ClusterSizeString.EndsWith(TEXT("KB")))
		{
			CmdLineParameters.FileSystemBlockSize *= 1024;
		}
	}
	else
	{
		CmdLineParameters.FileSystemBlockSize = 0;
	}

	FString CompBlockSizeString;
	if (FParse::Value(CmdLine, TEXT("-compressionblocksize="), CompBlockSizeString) &&
		FParse::Value(CmdLine, TEXT("-compressionblocksize="), CmdLineParameters.CompressionBlockSize))
	{
		if (CompBlockSizeString.EndsWith(TEXT("MB")))
		{
			CmdLineParameters.CompressionBlockSize *= 1024 * 1024;
		}
		else if (CompBlockSizeString.EndsWith(TEXT("KB")))
		{
			CmdLineParameters.CompressionBlockSize *= 1024;
		}
	}

	if (!FParse::Value(CmdLine, TEXT("-patchpaddingalign="), CmdLineParameters.PatchFilePadAlign))
	{
		CmdLineParameters.PatchFilePadAlign = 0;
	}

	if (!FParse::Value(CmdLine, TEXT("-AlignForMemoryMapping="), CmdLineParameters.AlignForMemoryMapping))
	{
		CmdLineParameters.AlignForMemoryMapping = 0;
	}

	if (FParse::Param(CmdLine, TEXT("encryptindex")))
	{
		CmdLineParameters.EncryptIndex = true;
	}

	if (FParse::Param(CmdLine, TEXT("sign")))
	{
		CmdLineParameters.bSign = true;
	}

	if (FParse::Param(CmdLine, TEXT("AlignFilesLargerThanBlock")))
	{
		CmdLineParameters.bAlignFilesLargerThanBlock = true;
	}

	if (FParse::Param(CmdLine, TEXT("ForceCompress")))
	{
		CmdLineParameters.bForceCompress = true;
		UE_LOGF(LogPakFile, Warning, "-ForceCompress is deprecated.  It will be removed in a future release.");
	}

	if (FParse::Param(CmdLine, TEXT("FileRegions")))
	{
		CmdLineParameters.bFileRegions = true;
	}

	FString DesiredCompressionFormats;
	// look for -compressionformats or -compressionformat on the commandline
	if (FParse::Value(CmdLine, TEXT("-compressionformats="), DesiredCompressionFormats) || FParse::Value(CmdLine, TEXT("-compressionformat="), DesiredCompressionFormats))
	{
		TArray<FString> Formats;
		DesiredCompressionFormats.ParseIntoArray(Formats, TEXT(","));
		for (FString& Format : Formats)
		{
			// look until we have a valid format
			FName FormatName = *Format;

			if (FCompression::IsFormatValid(FormatName))
			{
				CmdLineParameters.CompressionFormats.Add(FormatName);
				break;
			}
			else
			{
				UE_LOGF(LogPakFile, Warning, "Compression format %ls is not recognized", *Format);
			}
		}
	}

	// disable zlib as fallback if requested
	if (FParse::Param(CmdLine, TEXT("disablezlib")))
	{
		UE_LOGF(LogPakFile, Display, "Disabling ZLib as a compression option.");
	}
	else
	{
		CmdLineParameters.CompressionFormats.AddUnique(NAME_Zlib);
	}

	FParse::Value(CmdLine, TEXT("-patchSeekOptMaxInflationPercent="), CmdLineParameters.SeekOptParams.MaxInflationPercent);
	ReadSizeParam(CmdLine, TEXT("-patchSeekOptMaxGapSize="), CmdLineParameters.SeekOptParams.MaxGapSize);
	FParse::Value(CmdLine, TEXT("-patchSeekOptMaxAdjacentOrderDiff="), CmdLineParameters.SeekOptParams.MaxAdjacentOrderDiff);

	// For legacy reasons, if we specify a max gap size without a mode, we default to OnePass
	if (CmdLineParameters.SeekOptParams.MaxGapSize > 0)
	{
		CmdLineParameters.SeekOptParams.Mode = ESeekOptMode::OnePass;
	}
	FParse::Value(CmdLine, TEXT("-patchSeekOptMode="), (int32&)CmdLineParameters.SeekOptParams.Mode);

	FParse::Value(CmdLine, TEXT("csv="), CmdLineParameters.CsvPath);

	FParse::Value(FCommandLine::Get(), TEXT("ProjectStore="), CmdLineParameters.ProjectStoreFilename);
}

void ProcessPakFileSpecificCommandLine(const TCHAR* CmdLine, const TArray<FString>& NonOptionArguments, TArray<FPakInputPair>& Entries, FPakCommandLineParameters& CmdLineParameters)
{
	FString ResponseFile;
	if (FParse::Value(CmdLine, TEXT("-create="), ResponseFile))
	{
		CmdLineParameters.GeneratePatch = FParse::Value(CmdLine, TEXT("-generatepatch="), CmdLineParameters.SourcePatchPakFilename);
		FParse::Value(CmdLine, TEXT("-outputchangedfiles="), CmdLineParameters.ChangedFilesOutputFilename);

		bool bCompress = FParse::Param(CmdLine, TEXT("compress"));
		if ( bCompress )
		{
			// the correct way to enable compression is via bCompressed in UProjectPackagingSettings
			//	which passes -compressed to CopyBuildToStaging and writes the response file
			UE_LOGF(LogPakFile, Warning, "-compress is deprecated, use -compressed with UAT instead");
		}

		bool bEncrypt = FParse::Param(CmdLine, TEXT("encrypt"));		

		// if the response file is a pak file, then this is the pak file we want to use as the source
		if (ResponseFile.EndsWith(TEXT(".pak"), ESearchCase::IgnoreCase) && CmdLineParameters.GeneratePatch)
		{
			FString OutputPath;
			if (FParse::Value(CmdLine, TEXT("extractedpaktemp="), OutputPath) == false)
			{
				UE_LOGF(LogPakFile, Error, "-extractedpaktemp= not specified.  Required when specifying pak file as the response file.");
			}

			FString ExtractedPakKeysFile;
			FKeyChain ExtractedPakKeys;
			if ( FParse::Value(CmdLine, TEXT("extractedpakcryptokeys="), ExtractedPakKeysFile) )
			{
				KeyChainUtilities::LoadKeyChainFromFile(ExtractedPakKeysFile, ExtractedPakKeys);
				KeyChainUtilities::ApplyEncryptionKeys(ExtractedPakKeys);
			}

			TMap<FString, FFileInfo> FileHashes;
			ExtractFilesFromPak(*ResponseFile, FileHashes, *OutputPath, true, ExtractedPakKeys, nullptr, &Entries, nullptr, nullptr, nullptr, nullptr);
		}
		else
		{
			TArray<FString> Lines;
			bool bParseLines = true;
			if (IFileManager::Get().DirectoryExists(*ResponseFile))
			{
				IFileManager::Get().FindFilesRecursive(Lines, *ResponseFile, TEXT("*"), true, false);
				bParseLines = false;
			}
			else
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(LoadResponseFile);
				FString Text;
				UE_LOGF(LogPakFile, Display, "Loading response file %ls", *ResponseFile);
				if (FFileHelper::LoadFileToString(Text, *ResponseFile))
				{
					// Remove all carriage return characters.
					Text.ReplaceInline(TEXT("\r"), TEXT(""));
					// Read all lines
					Text.ParseIntoArray(Lines, TEXT("\n"), true);
				}
				else
				{
					UE_LOGF(LogPakFile, Error, "Failed to load %ls", *ResponseFile);
				}
			}

			TRACE_CPUPROFILER_EVENT_SCOPE(AddEntries);
			
			Entries.Reserve(Lines.Num());
			FString NextToken;
			for (int32 EntryIndex = 0; EntryIndex < Lines.Num(); EntryIndex++)
			{
				FPakInputPair& Input = Entries.AddDefaulted_GetRef();
				if (bParseLines)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(CommandLineParseHelper);
					bool bHasParsedSource = false;
					bool bHasParsedDest = false;
					const TCHAR* LinePtr = *Lines[EntryIndex];
					while (FParse::Token(LinePtr, NextToken, false))
					{
						FStringView TokenView(NextToken);
						if (TokenView[0] == TCHAR('-'))
						{
							FStringView Switch = TokenView.Mid(1);
							if (Switch == TEXT("compress"))
							{
								Input.bNeedsCompression = true;
							}
							else if (Switch == TEXT("encrypt"))
							{
								Input.bNeedEncryption = true;
							}
							else if (Switch == TEXT("delete"))
							{
								Input.bIsDeleteRecord = true;
							}
							else if (Switch == TEXT("rehydrate"))
							{
								Input.bNeedRehydration = true;
								CmdLineParameters.bRequiresRehydration = true;
							}
						}
						else
						{
							if (!bHasParsedSource)
							{
								Input.Source = MoveTemp(NextToken);
								bHasParsedSource = true;
							}
							else if (!bHasParsedDest)
							{
								Input.Dest = MoveTemp(NextToken);
								bHasParsedDest = true;
							}
						}
					}
					if (!bHasParsedSource)
					{
						Entries.Pop(EAllowShrinking::No);
						continue;
					}
					
					if (!bHasParsedDest || Input.bIsDeleteRecord)
					{
						Input.Dest = Input.Source;
					}
				}
				else
				{
					Input.Source = MoveTemp(Lines[EntryIndex]);
					Input.Dest = Input.Source;
				}

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(OtherPathStuff);
					FPaths::NormalizeFilename(Input.Source);
					FPaths::NormalizeFilename(Input.Dest);
				}

				Input.bNeedsCompression |= bCompress;
				Input.bNeedEncryption |= bEncrypt;

				UE_LOGF(LogPakFile, Verbose, "Added file Source: %ls Dest: %ls", *Input.Source, *Input.Dest);

				bool bIsMappedBulk = Input.Source.EndsWith(TEXT(".m.ubulk"));
				if (bIsMappedBulk && CmdLineParameters.AlignForMemoryMapping > 0 && Input.bNeedsCompression
					&& !Input.bNeedEncryption) // if it is encrypted, we will compress it anyway since it won't be mapped at runtime
				{
					// no compression for bulk aligned files because they are memory mapped
					Input.bNeedsCompression = false;
					UE_LOGF(LogPakFile, Verbose, "Stripped compression from %ls for memory mapping.", *Input.Dest);
				}
			}
		}

		UE_LOGF(LogPakFile, Display, "Added %d entries to add to pak file.", Entries.Num());
	}
	else
	{
		// Override destination path.
		FString MountPoint;
		FParse::Value(CmdLine, TEXT("-dest="), MountPoint);
		FPaths::NormalizeFilename(MountPoint);
		FPakFile::MakeDirectoryFromPath(MountPoint);

		// Parse command line params. The first param after the program name is the created pak name
		for (int32 Index = 1; Index < NonOptionArguments.Num(); Index++)
		{
			// Skip switches and add everything else to the Entries array
			FPakInputPair Input;
			Input.Source = *NonOptionArguments[Index];
			FPaths::NormalizeFilename(Input.Source);
			if (MountPoint.Len() > 0)
			{
				FString SourceDirectory( FPaths::GetPath(Input.Source) );
				FPakFile::MakeDirectoryFromPath(SourceDirectory);
				Input.Dest = Input.Source.Replace(*SourceDirectory, *MountPoint, ESearchCase::IgnoreCase);
			}
			else
			{
				Input.Dest = FPaths::GetPath(Input.Source);
				FPakFile::MakeDirectoryFromPath(Input.Dest);
			}
			FPaths::NormalizeFilename(Input.Dest);
			Entries.Add(Input);
		}
	}
}

void ProcessCommandLine(const TCHAR* CmdLine, const TArray<FString>& NonOptionArguments, TArray<FPakInputPair>& Entries, FPakCommandLineParameters& CmdLineParameters)
{
	ProcessCommonCommandLine(CmdLine, CmdLineParameters);
	ProcessPakFileSpecificCommandLine(CmdLine, NonOptionArguments, Entries, CmdLineParameters);
}

bool InitializeVirtualizationSystem()
{
	if (UE::Virtualization::IVirtualizationSystem::IsInitialized())
	{
		return true;
	}

	UE_LOGF(LogPakFile, Display, "Initializing the virtualization system...");

	if (FModuleManager::Get().LoadModule(TEXT("Virtualization"), ELoadModuleFlags::LogFailures) == nullptr)
	{
		UE_LOGF(LogPakFile, Error, "Failed to load the virtualization module");
		return false;
	}

	IPluginManager& PluginMgr = IPluginManager::Get();

	const FString PerforcePluginPath = FPaths::EnginePluginsDir() / TEXT("Developer/PerforceSourceControl/PerforceSourceControl.uplugin");
	FText ErrorMsg;
	if (!PluginMgr.AddToPluginsList(PerforcePluginPath, &ErrorMsg))
	{
		UE_LOGF(LogPakFile, Error, "Failed to find 'PerforceSourceControl' plugin due to: %ls", *ErrorMsg.ToString());
		return false;
	}

	PluginMgr.MountNewlyCreatedPlugin(TEXT("PerforceSourceControl"));

	TSharedPtr<IPlugin> Plugin = PluginMgr.FindPlugin(TEXT("PerforceSourceControl"));
	if (Plugin == nullptr || !Plugin->IsEnabled())
	{
		UE_LOGF(LogPakFile, Error, "The 'PerforceSourceControl' plugin is disabled.");
		return false;
	}

	FConfigFile Config;

	const FString ProjectPath = FPaths::GetPath(FPaths::GetProjectFilePath());
	const FString EngineConfigPath = FPaths::Combine(FPaths::EngineDir(), TEXT("Config/"));
	const FString ProjectConfigPath = FPaths::Combine(ProjectPath, TEXT("Config/"));

	if (!FConfigCacheIni::LoadExternalIniFile(Config, TEXT("Engine"), *EngineConfigPath, *ProjectConfigPath, true))
	{
		UE_LOGF(LogPakFile, Error, "Failed to load config files for the project '%ls", *ProjectPath);
		return false;
	}

		// We might need to make sure that the DDC is initialized too
#if !USE_DDC_FOR_COMPRESSED_FILES
	GetDerivedDataCacheRef();
#endif //!USE_DDC_FOR_COMPRESSED_FILES

	UE::Virtualization::FInitParams InitParams(FApp::GetProjectName(), Config);
	UE::Virtualization::Initialize(InitParams, UE::Virtualization::EInitializationFlags::ForceInitialize);

	UE_LOGF(LogPakFile, Display, "Virtualization system initialized");

	return true;
}


void CollectFilesToAdd(TArray<FPakInputPair>& OutFilesToAdd, const TArray<FPakInputPair>& InEntries, const FPakOrderMap& OrderMap, const FPakCommandLineParameters& CmdLineParameters)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CollectFilesToAdd);
	UE_LOGF(LogPakFile, Display, "Collecting files to add to pak file...");
	const double StartTime = FPlatformTime::Seconds();

	// Start collecting files
	TSet<FString> AddedFiles;	
	for (int32 Index = 0; Index < InEntries.Num(); Index++)
	{
		const FPakInputPair& Input = InEntries[Index];
		const FString& Source = Input.Source;
		bool bCompression = Input.bNeedsCompression;
		bool bEncryption = Input.bNeedEncryption;
		bool bRehydrate = Input.bNeedRehydration;

		if (Input.bIsDeleteRecord)
		{
			// just pass through any delete records found in the input
			OutFilesToAdd.Add(Input);
			continue;
		}

		FString Filename = FPaths::GetCleanFilename(Source);
		FString Directory = FPaths::GetPath(Source);
		FPaths::MakeStandardFilename(Directory);
		FPakFile::MakeDirectoryFromPath(Directory);

		if (Filename.IsEmpty())
		{
			Filename = TEXT("*.*");
		}
		if ( Filename.Contains(TEXT("*")) )
		{
			// Add multiple files
			TArray<FString> FoundFiles;
			IFileManager::Get().FindFilesRecursive(FoundFiles, *Directory, *Filename, true, false);

			FString DestDirectory = FPaths::GetPath(Input.Dest);
			for (int32 FileIndex = 0; FileIndex < FoundFiles.Num(); FileIndex++)
			{
				FPakInputPair FileInput;
				FileInput.Source = FoundFiles[FileIndex];
				FPaths::MakeStandardFilename(FileInput.Source);
				FileInput.Dest = FPaths::Combine(DestDirectory, FPaths::GetCleanFilename(FileInput.Source));
				
				uint64 FileOrder = OrderMap.GetFileOrder(FileInput.Dest, false, &FileInput.bIsInPrimaryOrder);
				if(FileOrder != MAX_uint64)
				{
					FileInput.SuggestedOrder = FileOrder;
				}
				else
				{
					// we will put all unordered files at 1 << 28 so that they are before any uexp or ubulk files we assign orders to here
					FileInput.SuggestedOrder = (1 << 28);
					// if this is a cook order or an old order it will not have uexp files in it, so we put those in the same relative order after all of the normal files, but before any ubulk files
					if (FileInput.Dest.EndsWith(TEXT("uexp")) || FileInput.Dest.EndsWith(TEXT("ubulk")))
					{
						FileOrder = OrderMap.GetFileOrder(FPaths::GetBaseFilename(FileInput.Dest, false) + TEXT(".uasset"), false, &FileInput.bIsInPrimaryOrder);
						if (FileOrder == MAX_uint64)
						{
							FileOrder = OrderMap.GetFileOrder(FPaths::GetBaseFilename(FileInput.Dest, false) + TEXT(".umap"),  false, &FileInput.bIsInPrimaryOrder);
						}
						if (FileInput.Dest.EndsWith(TEXT("uexp")))
						{
							FileInput.SuggestedOrder = ((FileOrder != MAX_uint64) ? FileOrder : 0) + (1 << 29);
						}
						else
						{
							FileInput.SuggestedOrder = ((FileOrder != MAX_uint64) ? FileOrder : 0) + (1 << 30);
						}
					}
				}
				FileInput.bNeedsCompression = bCompression;
				FileInput.bNeedEncryption = bEncryption;
				FileInput.bNeedRehydration = bRehydrate;

				if (!AddedFiles.Contains(FileInput.Source))
				{
					OutFilesToAdd.Add(FileInput);
					AddedFiles.Add(FileInput.Source);
				}
				else
				{
					int32 FoundIndex;
					OutFilesToAdd.Find(FileInput,FoundIndex);
					OutFilesToAdd[FoundIndex].bNeedEncryption |= bEncryption;
					OutFilesToAdd[FoundIndex].bNeedsCompression |= bCompression;
					OutFilesToAdd[FoundIndex].bNeedRehydration |= bRehydrate;
					OutFilesToAdd[FoundIndex].SuggestedOrder = FMath::Min<uint64>(OutFilesToAdd[FoundIndex].SuggestedOrder, FileInput.SuggestedOrder);
				}
			}
		}
		else
		{
			// Add single file
			FPakInputPair FileInput;
			FileInput.Source = Input.Source;
			FPaths::MakeStandardFilename(FileInput.Source);
			FileInput.Dest = Input.Dest.EndsWith(TEXT("/")) ?
				FileInput.Source.Replace(*Directory, *Input.Dest, ESearchCase::IgnoreCase) :
				Input.Dest;
			uint64 FileOrder = OrderMap.GetFileOrder(FileInput.Dest, CmdLineParameters.bFallbackOrderForNonUassetFiles, &FileInput.bIsInPrimaryOrder);
			if (FileOrder != MAX_uint64)
			{
				FileInput.SuggestedOrder = FileOrder;
			}
			FileInput.bNeedEncryption = bEncryption;
			FileInput.bNeedsCompression = bCompression;
			FileInput.bNeedRehydration = bRehydrate;

			if (AddedFiles.Contains(FileInput.Source))
			{
				int32 FoundIndex;
				OutFilesToAdd.Find(FileInput, FoundIndex);
				OutFilesToAdd[FoundIndex].bNeedEncryption |= bEncryption;
				OutFilesToAdd[FoundIndex].bNeedsCompression |= bCompression;
				OutFilesToAdd[FoundIndex].bNeedRehydration |= bRehydrate;
				OutFilesToAdd[FoundIndex].SuggestedOrder = FMath::Min<uint64>(OutFilesToAdd[FoundIndex].SuggestedOrder, FileInput.SuggestedOrder);
			}
			else
			{
				OutFilesToAdd.Add(FileInput);
				AddedFiles.Add(FileInput.Source);
			}
		}
	}

	// Sort by suggested order then alphabetically
	struct FInputPairSort
	{
		FORCEINLINE bool operator()(const FPakInputPair& A, const FPakInputPair& B) const
		{
			return  A.bIsDeleteRecord == B.bIsDeleteRecord ? (A.SuggestedOrder == B.SuggestedOrder ? A.Dest < B.Dest : A.SuggestedOrder < B.SuggestedOrder) : A.bIsDeleteRecord < B.bIsDeleteRecord;
		}
	};
	OutFilesToAdd.Sort(FInputPairSort());
	UE_LOGF(LogPakFile, Display, "Collected %d files in %.2lfs.", OutFilesToAdd.Num(), FPlatformTime::Seconds() - StartTime);
}

bool BufferedCopyFile(FArchive& Dest, FArchive& Source, const FPakFile& PakFile, const FPakEntry& Entry, void* Buffer, int64 BufferSize, const FKeyChain& InKeyChain)
{	
	// Align down
	BufferSize = BufferSize & ~(FAES::AESBlockSize-1);
	int64 RemainingSizeToCopy = Entry.Size;
	while (RemainingSizeToCopy > 0)
	{
		const int64 SizeToCopy = FMath::Min(BufferSize, RemainingSizeToCopy);
		// If file is encrypted so we need to account for padding
		int64 SizeToRead = Entry.IsEncrypted() ? Align(SizeToCopy,FAES::AESBlockSize) : SizeToCopy;

		Source.Serialize(Buffer,SizeToRead);
		if (Entry.IsEncrypted())
		{
			const FNamedAESKey* Key = InKeyChain.GetPrincipalEncryptionKey();
			check(Key);
			FAES::DecryptData((uint8*)Buffer, SizeToRead, Key->Key);
		}
		Dest.Serialize(Buffer, SizeToCopy);
		RemainingSizeToCopy -= SizeToRead;
	}
	return true;
}

bool UncompressCopyFile(FArchive& Dest, FArchive& Source, const FPakEntry& Entry, uint8*& PersistentBuffer, int64& BufferSize, const FKeyChain& InKeyChain, const FPakFile& PakFile)
{
	if (Entry.UncompressedSize == 0)
	{
		return false;
	}

	// The compression block size depends on the bit window that the PAK file was originally created with. Since this isn't stored in the PAK file itself,
	// we can use FCompression::CompressMemoryBound as a guideline for the max expected size to avoid unncessary reallocations, but we need to make sure
	// that we check if the actual size is not actually greater (eg. UE-59278).
	FName EntryCompressionMethod = PakFile.GetInfo().GetCompressionMethod(Entry.CompressionMethodIndex);
	int32 MaxCompressionBlockSize = FCompression::CompressMemoryBound(EntryCompressionMethod, Entry.CompressionBlockSize);
	for (const FPakCompressedBlock& Block : Entry.CompressionBlocks)
	{
		MaxCompressionBlockSize = FMath::Max<int32>(MaxCompressionBlockSize, IntCastChecked<int32>(Block.CompressedEnd - Block.CompressedStart));
	}

	int64 WorkingSize = Entry.CompressionBlockSize + MaxCompressionBlockSize;
	if (BufferSize < WorkingSize)
	{
		PersistentBuffer = (uint8*)FMemory::Realloc(PersistentBuffer, WorkingSize);
		BufferSize = WorkingSize;
	}

	uint8* UncompressedBuffer = PersistentBuffer+MaxCompressionBlockSize;

	for (uint32 BlockIndex=0, BlockIndexNum=Entry.CompressionBlocks.Num(); BlockIndex < BlockIndexNum; ++BlockIndex)
	{
		int64 CompressedBlockSize = Entry.CompressionBlocks[BlockIndex].CompressedEnd - Entry.CompressionBlocks[BlockIndex].CompressedStart;
		int64 UncompressedBlockSize = FMath::Min<int64>(Entry.UncompressedSize - Entry.CompressionBlockSize*BlockIndex, Entry.CompressionBlockSize);
		Source.Seek(Entry.CompressionBlocks[BlockIndex].CompressedStart + (PakFile.GetInfo().HasRelativeCompressedChunkOffsets() ? Entry.Offset : 0));
		int64 SizeToRead = Entry.IsEncrypted() ? Align(CompressedBlockSize, FAES::AESBlockSize) : CompressedBlockSize;
		Source.Serialize(PersistentBuffer, SizeToRead);

		if (Entry.IsEncrypted())
		{
			const FNamedAESKey* Key = InKeyChain.GetEncryptionKeys().Find(PakFile.GetInfo().EncryptionKeyGuid);
			if (Key == nullptr)
			{
				Key = InKeyChain.GetPrincipalEncryptionKey();
			}
			check(Key);
			FAES::DecryptData(PersistentBuffer, SizeToRead, Key->Key);
		}

		if (!FCompression::UncompressMemory(EntryCompressionMethod, UncompressedBuffer, IntCastChecked<int32>(UncompressedBlockSize), PersistentBuffer, IntCastChecked<int32>(CompressedBlockSize)))
		{
			return false;
		}
		Dest.Serialize(UncompressedBuffer,UncompressedBlockSize);
	}

	return true;
}

TEncryptionInt ParseEncryptionIntFromJson(TSharedPtr<FJsonObject> InObj, const TCHAR* InName)
{
	FString Base64;
	if (InObj->TryGetStringField(InName, Base64))
	{
		TArray<uint8> Bytes;
		FBase64::Decode(Base64, Bytes);
		check(Bytes.Num() == sizeof(TEncryptionInt));
		return TEncryptionInt((uint32*)&Bytes[0]);
	}
	else
	{
		return TEncryptionInt();
	}
}

void LoadKeyChain(const TCHAR* CmdLine, FKeyChain& OutCryptoSettings)
{
	OutCryptoSettings.SetSigningKey( InvalidRSAKeyHandle );
	OutCryptoSettings.GetEncryptionKeys().Empty();

	// First, try and parse the keys from a supplied crypto key cache file
	FString CryptoKeysCacheFilename;
	if (FParse::Value(CmdLine, TEXT("cryptokeys="), CryptoKeysCacheFilename))
	{
		UE_LOGF(LogPakFile, Display, "Parsing crypto keys from a crypto key cache file");
		KeyChainUtilities::LoadKeyChainFromFile(CryptoKeysCacheFilename, OutCryptoSettings);
	}
	else if (FParse::Param(CmdLine, TEXT("encryptionini")))
	{
		FString ProjectDir, EngineDir, Platform;

		if (FParse::Value(CmdLine, TEXT("projectdir="), ProjectDir, false)
			&& FParse::Value(CmdLine, TEXT("enginedir="), EngineDir, false)
			&& FParse::Value(CmdLine, TEXT("platform="), Platform, false))
		{
			UE_LOGF(LogPakFile, Warning, "A legacy command line syntax is being used for crypto config. Please update to using the -cryptokey parameter as soon as possible as this mode is deprecated");

			FConfigFile EngineConfig;

			FConfigCacheIni::LoadExternalIniFile(EngineConfig, TEXT("Engine"), *FPaths::Combine(EngineDir, TEXT("Config\\")), *FPaths::Combine(ProjectDir, TEXT("Config/")), true, *Platform);
			bool bDataCryptoRequired = false;
			EngineConfig.GetBool(TEXT("PlatformCrypto"), TEXT("PlatformRequiresDataCrypto"), bDataCryptoRequired);

			if (!bDataCryptoRequired)
			{
				return;
			}

			FConfigFile ConfigFile;
			FConfigCacheIni::LoadExternalIniFile(ConfigFile, TEXT("Crypto"), *FPaths::Combine(EngineDir, TEXT("Config\\")), *FPaths::Combine(ProjectDir, TEXT("Config/")), true, *Platform);
			bool bSignPak = false;
			bool bEncryptPakIniFiles = false;
			bool bEncryptPakIndex = false;
			bool bEncryptAssets = false;
			bool bEncryptPak = false;

			if (ConfigFile.Num())
			{
				UE_LOGF(LogPakFile, Display, "Using new format crypto.ini files for crypto configuration");

				static const TCHAR* SectionName = TEXT("/Script/CryptoKeys.CryptoKeysSettings");

				ConfigFile.GetBool(SectionName, TEXT("bEnablePakSigning"), bSignPak);
				ConfigFile.GetBool(SectionName, TEXT("bEncryptPakIniFiles"), bEncryptPakIniFiles);
				ConfigFile.GetBool(SectionName, TEXT("bEncryptPakIndex"), bEncryptPakIndex);
				ConfigFile.GetBool(SectionName, TEXT("bEncryptAssets"), bEncryptAssets);
				bEncryptPak = bEncryptPakIniFiles || bEncryptPakIndex || bEncryptAssets;

				if (bSignPak)
				{
					FString PublicExpBase64, PrivateExpBase64, ModulusBase64;
					ConfigFile.GetString(SectionName, TEXT("SigningPublicExponent"), PublicExpBase64);
					ConfigFile.GetString(SectionName, TEXT("SigningPrivateExponent"), PrivateExpBase64);
					ConfigFile.GetString(SectionName, TEXT("SigningModulus"), ModulusBase64);

					TArray<uint8> PublicExp, PrivateExp, Modulus;
					FBase64::Decode(PublicExpBase64, PublicExp);
					FBase64::Decode(PrivateExpBase64, PrivateExp);
					FBase64::Decode(ModulusBase64, Modulus);

					OutCryptoSettings.SetSigningKey(FRSA::CreateKey(PublicExp, PrivateExp, Modulus));

					UE_LOGF(LogPakFile, Display, "Parsed signature keys from config files.");
				}

				if (bEncryptPak)
				{
					FString EncryptionKeyString;
					ConfigFile.GetString(SectionName, TEXT("EncryptionKey"), EncryptionKeyString);

					if (EncryptionKeyString.Len() > 0)
					{
						TArray<uint8> Key;
						FBase64::Decode(EncryptionKeyString, Key);
						check(Key.Num() == sizeof(FAES::FAESKey::Key));
						FNamedAESKey NewKey;
						NewKey.Name = TEXT("Default");
						NewKey.Guid = FGuid();
						FMemory::Memcpy(NewKey.Key.Key, &Key[0], sizeof(FAES::FAESKey::Key));
						OutCryptoSettings.GetEncryptionKeys().Add(NewKey.Guid, NewKey);
						UE_LOGF(LogPakFile, Display, "Parsed AES encryption key from config files.");
					}
				}
			}
			else
			{
				static const TCHAR* SectionName = TEXT("Core.Encryption");

				UE_LOGF(LogPakFile, Display, "Using old format encryption.ini files for crypto configuration");

				FConfigCacheIni::LoadExternalIniFile(ConfigFile, TEXT("Encryption"), *FPaths::Combine(EngineDir, TEXT("Config\\")), *FPaths::Combine(ProjectDir, TEXT("Config/")), true, *Platform);
				ConfigFile.GetBool(SectionName, TEXT("SignPak"), bSignPak);
				ConfigFile.GetBool(SectionName, TEXT("EncryptPak"), bEncryptPak);

				if (bSignPak)
				{
					FString RSAPublicExp, RSAPrivateExp, RSAModulus;
					ConfigFile.GetString(SectionName, TEXT("rsa.publicexp"), RSAPublicExp);
					ConfigFile.GetString(SectionName, TEXT("rsa.privateexp"), RSAPrivateExp);
					ConfigFile.GetString(SectionName, TEXT("rsa.modulus"), RSAModulus);

					//TODO: Fix me!
					//OutSigningKey.PrivateKey.Exponent.Parse(RSAPrivateExp);
					//OutSigningKey.PrivateKey.Modulus.Parse(RSAModulus);
					//OutSigningKey.PublicKey.Exponent.Parse(RSAPublicExp);
					//OutSigningKey.PublicKey.Modulus = OutSigningKey.PrivateKey.Modulus;

					UE_LOGF(LogPakFile, Display, "Parsed signature keys from config files.");
				}

				if (bEncryptPak)
				{
					FString EncryptionKeyString;
					ConfigFile.GetString(SectionName, TEXT("aes.key"), EncryptionKeyString);
					FNamedAESKey NewKey;
					NewKey.Name = TEXT("Default");
					NewKey.Guid = FGuid();
					if (EncryptionKeyString.Len() == 32 && TCString<TCHAR>::IsPureAnsi(*EncryptionKeyString))
					{
						for (int32 Index = 0; Index < 32; ++Index)
						{
							NewKey.Key.Key[Index] = (uint8)EncryptionKeyString[Index];
						}
						OutCryptoSettings.GetEncryptionKeys().Add(NewKey.Guid, NewKey);
						UE_LOGF(LogPakFile, Display, "Parsed AES encryption key from config files.");
					}
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogPakFile, Display, "Using command line for crypto configuration");

		FString EncryptionKeyString;
		FParse::Value(CmdLine, TEXT("aes="), EncryptionKeyString, false);

		if (EncryptionKeyString.Len() > 0)
		{
			UE_LOGF(LogPakFile, Warning, "A legacy command line syntax is being used for crypto config. Please update to using the -cryptokey parameter as soon as possible as this mode is deprecated");

			FNamedAESKey NewKey;
			NewKey.Name = TEXT("Default");
			NewKey.Guid = FGuid();
			const uint32 RequiredKeyLength = sizeof(NewKey.Key);

			// Error checking
			if (EncryptionKeyString.Len() < RequiredKeyLength)
			{
				UE_LOGF(LogPakFile, Fatal, "AES encryption key must be %d characters long", RequiredKeyLength);
			}

			if (EncryptionKeyString.Len() > RequiredKeyLength)
			{
				UE_LOGF(LogPakFile, Warning, "AES encryption key is more than %d characters long, so will be truncated!", RequiredKeyLength);
				EncryptionKeyString.LeftInline(RequiredKeyLength);
			}

			if (!FCString::IsPureAnsi(*EncryptionKeyString))
			{
				UE_LOGF(LogPakFile, Fatal, "AES encryption key must be a pure ANSI string!");
			}

			const auto AsAnsi = StringCast<ANSICHAR>(*EncryptionKeyString);
			check(AsAnsi.Length() == RequiredKeyLength);
			FMemory::Memcpy(NewKey.Key.Key, AsAnsi.Get(), RequiredKeyLength);
			OutCryptoSettings.GetEncryptionKeys().Add(NewKey.Guid, NewKey);
			UE_LOGF(LogPakFile, Display, "Parsed AES encryption key from command line.");
		}
	}

	FString EncryptionKeyOverrideGuidString;
	FGuid EncryptionKeyOverrideGuid;
	if (FParse::Value(CmdLine, TEXT("EncryptionKeyOverrideGuid="), EncryptionKeyOverrideGuidString))
	{
		FGuid::Parse(EncryptionKeyOverrideGuidString, EncryptionKeyOverrideGuid);
	}
	OutCryptoSettings.SetPrincipalEncryptionKey(OutCryptoSettings.GetEncryptionKeys().Find(EncryptionKeyOverrideGuid));
}

/**
 * Creates a pak file writer. This can be a signed writer if the encryption keys are specified in the command line
 */
FArchive* CreatePakWriter(const TCHAR* Filename, const FKeyChain& InKeyChain, bool bSign)
{
	if (IFileManager::Get().FileExists(Filename))
	{
		UE_LOGF(LogPakFile, Error, "Failed to create pak at %ls. File already exists at that location.", Filename);
		return nullptr;
	}

	FArchive* Writer = IFileManager::Get().CreateFileWriter(Filename);

	if (Writer)
	{
		if (bSign)
		{
			UE_LOGF(LogPakFile, Display, "Creating signed pak %ls.", Filename);
			Writer = new FSignedArchiveWriter(*Writer, Filename, InKeyChain.GetSigningKey());
		}
		else
		{
			UE_LOGF(LogPakFile, Display, "Creating pak %ls.", Filename);
		}
	}

	return Writer;
}

/* Helper for index creation in CreatePakFile.  Used to (conditionally) write Bytes into a SecondaryIndex and serialize into the PrimaryIndex the offset of the SecondaryIndex */
struct FSecondaryIndexWriter
{
private:
	TArray<uint8>& SecondaryIndexData;
	FMemoryWriter SecondaryWriter;
	TArray<uint8>& PrimaryIndexData;
	FMemoryWriter& PrimaryIndexWriter;
	FSHAHash SecondaryHash;
	int64 OffsetToDataInPrimaryIndex = INDEX_NONE;
	int64 OffsetToSecondaryInPakFile = INDEX_NONE;
	int64 SecondarySize = 0;
	bool bShouldWrite;

public:
	FSecondaryIndexWriter(TArray<uint8>& InSecondaryIndexData, bool bInShouldWrite, TArray<uint8>& InPrimaryIndexData, FMemoryWriter& InPrimaryIndexWriter)
		: SecondaryIndexData(InSecondaryIndexData)
		, SecondaryWriter(SecondaryIndexData)
		, PrimaryIndexData(InPrimaryIndexData)
		, PrimaryIndexWriter(InPrimaryIndexWriter)
		, bShouldWrite(bInShouldWrite)
	{
		if (bShouldWrite)
		{
			SecondaryWriter.SetByteSwapping(PrimaryIndexWriter.ForceByteSwapping());
		}
	}

	/**
	 * Write the condition flag and the Offset,Size,Hash into the primary index.  Offset of the Secondary index cannot be calculated until the PrimaryIndex is done writing later,
	 * so placeholders are left instead, with a marker to rewrite them later.
	 */
	void WritePlaceholderToPrimary()
	{
		PrimaryIndexWriter << bShouldWrite;
		if (bShouldWrite)
		{
			OffsetToDataInPrimaryIndex = PrimaryIndexData.Num();
			PrimaryIndexWriter << OffsetToSecondaryInPakFile;
			PrimaryIndexWriter << SecondarySize;
			PrimaryIndexWriter << SecondaryHash;
		}
	}

	FMemoryWriter& GetSecondaryWriter()
	{
		return SecondaryWriter;
	}

	/** The caller has populated this->SecondaryIndexData using GetSecondaryWriter().  We now calculate the size, encrypt, and hash, and store the Offset,Size,Hash in the PrimaryIndex */
	void FinalizeAndRecordOffset(int64 OffsetInPakFile, const TFunction<void(TArray<uint8> & IndexData, FSHAHash & OutHash)>& FinalizeIndexBlock)
	{
		if (!bShouldWrite)
		{
			return;
		}

		FinalizeIndexBlock(SecondaryIndexData, SecondaryHash);
		SecondarySize = SecondaryIndexData.Num();
		OffsetToSecondaryInPakFile = OffsetInPakFile;

		PrimaryIndexWriter.Seek(OffsetToDataInPrimaryIndex);
		PrimaryIndexWriter << OffsetToSecondaryInPakFile;
		PrimaryIndexWriter << SecondarySize;
		PrimaryIndexWriter << SecondaryHash;
	}
};

/* Verify that Indexes constructed for serialization into the PakFile match the originally collected list of FPakEntryPairs */
void VerifyIndexesMatch(TArray<FPakEntryPair>& EntryList, FPakFile::FDirectoryIndex& DirectoryIndex, FPakFile::FPathHashIndex& PathHashIndex, uint64 PathHashSeed, const FString& MountPoint,
	const TArray<uint8>& EncodedPakEntries, const TArray<FPakEntry>& NonEncodableEntries, int32 NumEncodedEntries, int32 NumDeletedEntries, FPakInfo& Info)
{
	check(NumEncodedEntries + NonEncodableEntries.Num() + NumDeletedEntries == EntryList.Num());

	FPakEntry EncodedEntry;
	for (FPakEntryPair& Pair : EntryList)
	{
		FString FullPath = FPaths::Combine(MountPoint, Pair.Filename);

		const FPakEntryLocation* PakEntryLocation = FPakFile::FindLocationFromIndex(FullPath, MountPoint, DirectoryIndex);
		if (!PakEntryLocation)
		{
			check(false);
			continue;
		}
		const FPakEntryLocation* PathHashLocation = FPakFile::FindLocationFromIndex(FullPath, MountPoint, PathHashIndex, PathHashSeed, Info.Version);
		if (!PathHashLocation)
		{
			check(false);
			continue;
		}
		check(*PakEntryLocation == *PathHashLocation);

		check(FPakFile::GetPakEntry(*PakEntryLocation, &EncodedEntry, EncodedPakEntries, NonEncodableEntries, Info) != FPakFile::EFindResult::NotFound);
		check(Pair.Info.IsDeleteRecord() == EncodedEntry.IsDeleteRecord());
		check(Pair.Info.IsDeleteRecord() || Pair.Info.IndexDataEquals(EncodedEntry));
	}
};

class FPakWriterContext
{
public:
	bool Initialize(const FPakCommandLineParameters& InCmdLineParameters);
	bool AddPakFile(const TCHAR* Filename, const TArray<FPakInputPair>& FilesToAdd, const FKeyChain& InKeyChain);
	bool Flush();

private:
	struct FOutputPakFile;

	struct FOutputPakFileEntry
	{
		FOutputPakFile* PakFile = nullptr;
		FPakInputPair InputPair;
		int64 OriginalFileSize = 0;
		FOutputPakFileEntry* UExpFileInPair = nullptr;
		bool bIsUAssetUExpPairUAsset = false;
		bool bIsUAssetUExpPairUExp = false;
		bool bIsMappedBulk = false;
		bool bIsLastFileInPak = false;
		bool bSomeCompressionSucceeded = false;
		bool bCopiedToPak = false;

		FPakEntryPair Entry;
		
		FName CompressionMethod = NAME_None;
		int64 RealFileSize = 0;

		TUniquePtr<FMemoryCompressor> MemoryCompressor;
		FGraphEventRef BeginCompressionBarrier;
		FGraphEventRef BeginCompressionTask;
		FGraphEventRef EndCompressionBarrier;
		FGraphEventRef EndCompressionTask;

		FCompressedFileBuffer& AccessCompressedBuffer()
		{
			return CompressedFileBuffer;
		}

		const FCompressedFileBuffer& GetCompressedBuffer() const
		{
			return CompressedFileBuffer;
		}

	private:
		FCompressedFileBuffer CompressedFileBuffer;
	};

	struct FOutputPakFile
	{
		FString Filename;
		FKeyChain KeyChain;
		TUniquePtr<FArchive> PakFileHandle;
		TUniquePtr<FArchive> PakFileRegionsHandle;
		TArray<FOutputPakFileEntry> Entries;
		FPakInfo Info;
		FString MountPoint;
		TArray<FPakEntryPair> Index;
		// Some platforms provide patch download size reduction by diffing the patch files.  However, they often operate on specific block
		// sizes when dealing with new data within the file.  Pad files out to the given alignment to work with these systems more nicely.
		// We also want to combine smaller files into the same padding size block so we don't waste as much space. i.e. grouping 64 1k files together
		// rather than padding each out to 64k.
		uint64 ContiguousTotalSizeSmallerThanBlockSize = 0;
		uint64 ContiguousFilesSmallerThanBlockSize = 0;
		TArray<FFileRegion> AllFileRegions;
		// Stats
		uint64 TotalUncompressedSize = 0;
		uint64 TotalCompressedSize = 0;
		TArray<int32> Compressor_Stat_Count;
		TArray<uint64> Compressor_Stat_RawBytes;
		TArray<uint64> Compressor_Stat_CompBytes;
		uint64 TotalRequestedEncryptedFiles = 0;
		uint64 TotalEncryptedFiles = 0;
		uint64 TotalEncryptedDataSize = 0;
		FEventRef AllEntriesRetiredEvent;
		int32 RetiredEntriesCount = 0;
		uint64 RehydratedCount = 0;
		uint64 RehydratedBytes = 0;
	};

	void BeginCompress(FOutputPakFileEntry* Entry);
	void EndCompress(FOutputPakFileEntry* Entry);
	void Retire(FOutputPakFileEntry* Entry);
	void CompressionThreadFunc();
	void WriterThreadFunc();

	FPakCommandLineParameters CmdLineParameters;
	TSet<FString> NoPluginCompressionFileNames;
	TSet<FString> NoPluginCompressionExtensions;
	TArray64<uint8> PaddingBuffer;
	TArray<TUniquePtr<FOutputPakFile>> OutputPakFiles;
	TArray<FName> CompressionFormatsAndNone;
	TAtomic<int64> TotalFilesWithPoorForcedCompression{ 0 };
	TAtomic<int64> TotalExtraMemoryForPoorForcedCompression{ 0 };
	TSpscQueue<FOutputPakFileEntry*> CompressionQueue;
	TSpscQueue<FOutputPakFileEntry*> WriteQueue;
	TFuture<void> CompressionThread;
	TFuture<void> WriterThread;
	TUniquePtr<FCookedPackageStore> PackageStore;
	FEventRef CompressionQueueEntryAddedEvent;
	FEventRef WriteQueueEntryAddedEvent;
	FEventRef EntryRetiredEvent;
	TAtomic<int64> ScheduledFileSize{ 0 };
	TAtomic<bool> bFlushed{ false };
	uint64 TotalEntriesCount = 0;
	TAtomic<uint64> RetiredEntriesCount{ 0 };
};

bool FPakWriterContext::Initialize(const FPakCommandLineParameters& InCmdLineParameters)
{
#if USE_DDC_FOR_COMPRESSED_FILES
	GetDerivedDataCacheRef();
#endif

	CmdLineParameters = InCmdLineParameters;

	int64 PaddingBufferSize = 64 * 1024;
	PaddingBufferSize = FMath::Max(PaddingBufferSize, CmdLineParameters.AlignForMemoryMapping);
	PaddingBufferSize = FMath::Max(PaddingBufferSize, CmdLineParameters.PatchFilePadAlign);
	PaddingBuffer.SetNumZeroed(PaddingBufferSize);

	// track compression stats per format :
	CompressionFormatsAndNone = CmdLineParameters.CompressionFormats;
	CompressionFormatsAndNone.AddUnique(NAME_None);

	{
		// log the methods and indexes
		// we're going to prefer to use only index [0] so we want that to be our favorite compressor
		FString FormatLogLine(TEXT("CompressionFormats in priority order: "));
		for (int32 MethodIndex = 0; MethodIndex < CmdLineParameters.CompressionFormats.Num(); MethodIndex++)
		{
			FName CompressionMethod = CompressionFormatsAndNone[MethodIndex];
			if ( MethodIndex > 0 )
			{
				FormatLogLine += TEXT(", ");
			}
			FormatLogLine += CompressionMethod.ToString();
		}
		UE_LOGF(LogPakFile, Display, "%ls", *FormatLogLine);
	}

	if (!CmdLineParameters.ProjectStoreFilename.IsEmpty())
	{
		TUniquePtr<FCookedPackageStore> NewPackageStore = MakeUnique<FCookedPackageStore>(FPaths::GetPath(CmdLineParameters.ProjectStoreFilename));
		FIoStatus Status = NewPackageStore->LoadProjectStore(*CmdLineParameters.ProjectStoreFilename);
		if (Status.IsOk())
		{
			PackageStore = MoveTemp(NewPackageStore);
		}
		else
		{
			UE_LOGF(LogPakFile, Fatal, "Failed loading project store '%ls'", *CmdLineParameters.ProjectStoreFilename);
		}
	}

	// Oodle is built into the Engine now and can be used to decode startup phase files (ini,res,uplugin)
	//	 which used to be forced to Zlib
	//	 set bDoUseOodleDespiteNoPluginCompression = true to let Oodle compress those files
	bool bDoUseOodleDespiteNoPluginCompression = false;
	GConfig->GetBool(TEXT("Pak"), TEXT("bDoUseOodleDespiteNoPluginCompression"), bDoUseOodleDespiteNoPluginCompression, GEngineIni);

	if ( bDoUseOodleDespiteNoPluginCompression && CompressionFormatsAndNone[0] == NAME_Oodle )
	{
		// NoPluginCompression sets are empty

		UE_LOGF(LogPakFile, Display, "Oodle enabled on 'NoPluginCompression' files");
	}
	else
	{
		TArray<FString> ExtensionsToNotUsePluginCompression;
		GConfig->GetArray(TEXT("Pak"), TEXT("ExtensionsToNotUsePluginCompression"), ExtensionsToNotUsePluginCompression, GEngineIni);
		for (const FString& Ext : ExtensionsToNotUsePluginCompression)
		{
			NoPluginCompressionExtensions.Add(Ext);
		}
	
		TArray<FString> FileNamesToNotUsePluginCompression;
		GConfig->GetArray(TEXT("Pak"), TEXT("FileNamesToNotUsePluginCompression"), FileNamesToNotUsePluginCompression, GEngineIni);
		for (const FString& FileName : FileNamesToNotUsePluginCompression)
		{
			NoPluginCompressionFileNames.Add(FileName);
		}
	}

	CompressionThread = Async(EAsyncExecution::Thread, [this]()
		{
			CompressionThreadFunc();
		});

	WriterThread = Async(EAsyncExecution::Thread, [this]()
		{
			WriterThreadFunc();
		});



	return true;
}

bool FPakWriterContext::AddPakFile(const TCHAR* Filename, const TArray<FPakInputPair>& FilesToAdd, const FKeyChain& InKeyChain)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AddPakFile);
	check(!bFlushed);

	TUniquePtr<FOutputPakFile> OutputPakFile = MakeUnique<FOutputPakFile>();

	OutputPakFile->Filename = Filename;
	OutputPakFile->KeyChain = InKeyChain;
	
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CreateFileHandle);
		OutputPakFile->PakFileHandle.Reset(CreatePakWriter(Filename, InKeyChain, CmdLineParameters.bSign));
	}
	if (!OutputPakFile->PakFileHandle)
	{
		UE_LOGF(LogPakFile, Error, "Unable to create pak file \"%ls\".", Filename);
		return false;
	}

	if (CmdLineParameters.bFileRegions)
	{
		FString RegionsFilename = FString(Filename) + FFileRegion::RegionsFileExtension;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(CreateRegionsFileHandle);
			OutputPakFile->PakFileRegionsHandle.Reset(IFileManager::Get().CreateFileWriter(*RegionsFilename));
		}
		if (!OutputPakFile->PakFileRegionsHandle)
		{
			UE_LOGF(LogPakFile, Error, "Unable to create pak regions file \"%ls\".", *RegionsFilename);
			return false;
		}
	}

	if (InKeyChain.GetPrincipalEncryptionKey())
	{
		UE_LOGF(LogPakFile, Display, "Using encryption key '%ls' [%ls]", *InKeyChain.GetPrincipalEncryptionKey()->Name, *InKeyChain.GetPrincipalEncryptionKey()->Guid.ToString());
	}

	OutputPakFile->Info.bEncryptedIndex = (InKeyChain.GetPrincipalEncryptionKey() && CmdLineParameters.EncryptIndex);
	OutputPakFile->Info.EncryptionKeyGuid = InKeyChain.GetPrincipalEncryptionKey() ? InKeyChain.GetPrincipalEncryptionKey()->Guid : FGuid();

	if (CmdLineParameters.bPatchCompatibilityMode421)
	{
		// for old versions, put in some known names that we may have used
		OutputPakFile->Info.GetCompressionMethodIndex(NAME_None);
		OutputPakFile->Info.GetCompressionMethodIndex(NAME_Zlib);
		OutputPakFile->Info.GetCompressionMethodIndex(NAME_Gzip);
		OutputPakFile->Info.GetCompressionMethodIndex(TEXT("Bogus"));
		OutputPakFile->Info.GetCompressionMethodIndex(TEXT("Oodle"));
	}

	OutputPakFile->Compressor_Stat_Count.SetNumZeroed(CompressionFormatsAndNone.Num());
	OutputPakFile->Compressor_Stat_RawBytes.SetNumZeroed(CompressionFormatsAndNone.Num());
	OutputPakFile->Compressor_Stat_CompBytes.SetNumZeroed(CompressionFormatsAndNone.Num());

	OutputPakFile->MountPoint = GetCommonRootPath(FilesToAdd);

	OutputPakFile->Entries.SetNum(FilesToAdd.Num());
	for (int32 FileIndex = 0; FileIndex < FilesToAdd.Num(); FileIndex++)
	{
		FOutputPakFileEntry& OutputEntry = OutputPakFile->Entries[FileIndex];
		OutputEntry.PakFile = OutputPakFile.Get();
		OutputEntry.InputPair = FilesToAdd[FileIndex];

		OutputEntry.bIsMappedBulk = FilesToAdd[FileIndex].Source.EndsWith(TEXT(".m.ubulk"));
		OutputEntry.bIsLastFileInPak = FileIndex + 1 == FilesToAdd.Num();
		if (FileIndex > 0)
		{
			if (FPaths::GetBaseFilename(FilesToAdd[FileIndex - 1].Dest, false) == FPaths::GetBaseFilename(FilesToAdd[FileIndex].Dest, false) &&
				FPaths::GetExtension(FilesToAdd[FileIndex - 1].Dest, true) == TEXT(".uasset") &&
				FPaths::GetExtension(FilesToAdd[FileIndex].Dest, true) == TEXT(".uexp"))
			{
				OutputEntry.bIsUAssetUExpPairUExp = true;
			}
		}
		if (!OutputEntry.bIsUAssetUExpPairUExp && FileIndex + 1 < FilesToAdd.Num())
		{
			if (FPaths::GetBaseFilename(FilesToAdd[FileIndex].Dest, false) == FPaths::GetBaseFilename(FilesToAdd[FileIndex + 1].Dest, false) &&
				FPaths::GetExtension(FilesToAdd[FileIndex].Dest, true) == TEXT(".uasset") &&
				FPaths::GetExtension(FilesToAdd[FileIndex + 1].Dest, true) == TEXT(".uexp"))
			{
				OutputEntry.bIsUAssetUExpPairUAsset = true;
				OutputEntry.UExpFileInPair = &OutputPakFile->Entries[FileIndex + 1];
			}
		}
		OutputEntry.BeginCompressionBarrier = FGraphEvent::CreateGraphEvent();
		OutputEntry.BeginCompressionTask = FFunctionGraphTask::CreateAndDispatchWhenReady([this, &OutputEntry]()
			{
				BeginCompress(&OutputEntry);
			}, TStatId(), OutputEntry.BeginCompressionBarrier, ENamedThreads::AnyHiPriThreadHiPriTask);
		OutputEntry.EndCompressionBarrier = FGraphEvent::CreateGraphEvent();
		OutputEntry.EndCompressionTask = FFunctionGraphTask::CreateAndDispatchWhenReady([this, &OutputEntry]()
			{
				EndCompress(&OutputEntry);
			}, TStatId(), OutputEntry.EndCompressionBarrier, ENamedThreads::AnyHiPriThreadHiPriTask);
		CompressionQueue.Enqueue(&OutputEntry);
		WriteQueue.Enqueue(&OutputEntry);
	}

	CompressionQueueEntryAddedEvent->Trigger();
	WriteQueueEntryAddedEvent->Trigger();

	TotalEntriesCount += OutputPakFile->Entries.Num();
	if (OutputPakFile->Entries.IsEmpty())
	{
		OutputPakFile->AllEntriesRetiredEvent->Trigger();
	}
	OutputPakFiles.Add(MoveTemp(OutputPakFile));

	return true;
}

void FPakWriterContext::BeginCompress(FOutputPakFileEntry* Entry)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(BeginCompress);

	if (!Entry->AccessCompressedBuffer().ReadSource(Entry->InputPair, PackageStore.Get()))
	{
		// TODO: Should we give an error?
		Entry->bSomeCompressionSucceeded = true; // Prevent loading in EndCompress
		Entry->EndCompressionBarrier->DispatchSubsequents();
		return;
	}

	// don't try to compress tiny files
	// even if they do compress, it is a bad use of decoder time
	if (Entry->AccessCompressedBuffer().GetFileSize() < 1024 || !Entry->InputPair.bNeedsCompression)
	{	
		Entry->AccessCompressedBuffer().SetSourceAsWorkingBuffer();

		Entry->bSomeCompressionSucceeded = true; // Only used for logging purposes
		Entry->EndCompressionBarrier->DispatchSubsequents();
		// TODO: Set Entry->RealFileSize or change to accessor?
		return;
	}

	// if first method (oodle) doesn't compress, don't try other methods (zlib)
	// if Oodle refused to compress it was not an error, it was a choice because Oodle didn't think
	//  compressing that file was worth the decode time
	//	we do NOT want Zlib to then get enabled for that file!
	const int32 MethodIndex = 0;
	{
		Entry->CompressionMethod = CmdLineParameters.CompressionFormats[MethodIndex];

		// because compression is a plugin, certain files need to be loadable out of pak files before plugins are loadable
		// (like .uplugin files). for these, we enforce a non-plugin compression - zlib
		//	note that those file types are also excluded from iostore, so still go through this pak system
		if (NoPluginCompressionExtensions.Find(FPaths::GetExtension(Entry->InputPair.Source)) != nullptr ||
			NoPluginCompressionFileNames.Find(FPaths::GetCleanFilename(Entry->InputPair.Source)) != nullptr)
		{
			Entry->CompressionMethod = NAME_Zlib;
		}
	}
	// attempt to compress the data
	Entry->MemoryCompressor = Entry->AccessCompressedBuffer().BeginCompressFileToWorkingBuffer(Entry->InputPair, Entry->CompressionMethod, CmdLineParameters.CompressionBlockSize, Entry->EndCompressionBarrier);
	if (Entry->MemoryCompressor != nullptr)
	{
		Entry->MemoryCompressor->StartWork();
	}
}

void FPakWriterContext::EndCompress(FOutputPakFileEntry* Entry)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(EndCompress);

	const int32 MethodIndex = 0;
	{
		if (Entry->MemoryCompressor && Entry->AccessCompressedBuffer().EndCompressFileToWorkingBuffer(Entry->InputPair, Entry->CompressionMethod, CmdLineParameters.CompressionBlockSize, *Entry->MemoryCompressor.Get()))
		{
			// for modern compressors we don't want any funny heuristics turning compression on/off
			// let the compressor decide; it has an introspective measure of whether compression is worth doing or not
			bool bNotEnoughCompression = Entry->GetCompressedBuffer().TotalCompressedSize >= Entry->OriginalFileSize;

			if (Entry->CompressionMethod == NAME_Zlib)
			{
				// for forced-Zlib files still use the old heuristic :

				// Zlib must save at least 1K regardless of percentage (for small files)
				bNotEnoughCompression = (Entry->OriginalFileSize - Entry->GetCompressedBuffer().TotalCompressedSize) < 1024;
				if (!bNotEnoughCompression)
				{
					// Check the compression ratio, if it's too low just store uncompressed. Also take into account read size
					// if we still save 64KB it's probably worthwhile compressing, as that saves a file read operation in the runtime.
					// TODO: drive this threshold from the command line
					float PercentLess = ((float)Entry->GetCompressedBuffer().TotalCompressedSize / ((float)Entry->OriginalFileSize / 100.f));
					bNotEnoughCompression = (PercentLess > 90.f) && ((Entry->OriginalFileSize - Entry->GetCompressedBuffer().TotalCompressedSize) < 65536);
				}
			}

			const bool bIsLastCompressionFormat = MethodIndex == CmdLineParameters.CompressionFormats.Num() - 1;
			if (bNotEnoughCompression && (!CmdLineParameters.bForceCompress || !bIsLastCompressionFormat))
			{
				// compression did not succeed, we can try the next format, so do nothing here
			}
			else
			{
				Entry->Entry.Info.CompressionBlocks.AddUninitialized(Entry->GetCompressedBuffer().CompressedBlocks.Num());
				Entry->RealFileSize = Entry->GetCompressedBuffer().TotalCompressedSize + Entry->Entry.Info.GetSerializedSize(FPakInfo::PakFile_Version_Latest);
				Entry->Entry.Info.CompressionBlocks.Reset();

				// at this point, we have successfully compressed the file, no need to continue
				Entry->bSomeCompressionSucceeded = true;

				if (bNotEnoughCompression)
				{
					// None of the compression formats were good enough, but we were under instructions to use some form of compression
					// This was likely to aid with runtime error checking on mobile devices. Record how many cases of this we encountered and
					// How much the size difference was
					TotalFilesWithPoorForcedCompression++;
					TotalExtraMemoryForPoorForcedCompression += FMath::Max(Entry->GetCompressedBuffer().TotalCompressedSize - Entry->OriginalFileSize, (int64)0);
				}
			}
		}
	}
	Entry->MemoryCompressor.Reset();

	// If no compression was able to make it small enough, or compress at all, don't compress it
	if (Entry->bSomeCompressionSucceeded)
	{
		Entry->AccessCompressedBuffer().ResetSource();
	}
	else
	{
		Entry->CompressionMethod = NAME_None;
		Entry->AccessCompressedBuffer().SetSourceAsWorkingBuffer();
	}
}

void FPakWriterContext::Retire(FOutputPakFileEntry* Entry)
{
	++RetiredEntriesCount;
	Entry->AccessCompressedBuffer().Empty();

	ScheduledFileSize -= Entry->OriginalFileSize;
	EntryRetiredEvent->Trigger();
	++Entry->PakFile->RetiredEntriesCount;
	if (Entry->PakFile->RetiredEntriesCount == Entry->PakFile->Entries.Num())
	{
		Entry->PakFile->AllEntriesRetiredEvent->Trigger();
	}
}

void FPakWriterContext::CompressionThreadFunc()
{
	const int64 MaxScheduledFileSize = 2ll << 30;
	for (;;)
	{
		TOptional<FOutputPakFileEntry*> FromQueue = CompressionQueue.Dequeue();
		if (FromQueue.IsSet())
		{
			FOutputPakFileEntry* Entry = FromQueue.GetValue();
			if (!Entry->OriginalFileSize)
			{
				Entry->OriginalFileSize = IFileManager::Get().FileSize(*Entry->InputPair.Source);
			}
			Entry->RealFileSize = Entry->OriginalFileSize + Entry->Entry.Info.GetSerializedSize(FPakInfo::PakFile_Version_Latest);
			if (Entry->UExpFileInPair)
			{
				// Need the size of the uexp as well before we can proceed with this entry
				Entry->UExpFileInPair->OriginalFileSize = IFileManager::Get().FileSize(*Entry->UExpFileInPair->InputPair.Source);
			}

			int64 LocalScheduledFileSize = ScheduledFileSize;
			if (LocalScheduledFileSize + Entry->OriginalFileSize > MaxScheduledFileSize)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(WaitForMemory);
				while (LocalScheduledFileSize > 0 && LocalScheduledFileSize + Entry->OriginalFileSize > MaxScheduledFileSize)
				{
					EntryRetiredEvent->Wait();
					LocalScheduledFileSize = ScheduledFileSize;
				}
			}
			ScheduledFileSize += Entry->OriginalFileSize;
			Entry->BeginCompressionBarrier->DispatchSubsequents();
		}
		else
		{
			if (bFlushed)
			{
				return;
			}
			TRACE_CPUPROFILER_EVENT_SCOPE(WaitingForWork);
			CompressionQueueEntryAddedEvent->Wait();
		}
	}
}

void FPakWriterContext::WriterThreadFunc()
{
	const int64 RequiredPatchPadding = CmdLineParameters.PatchFilePadAlign;

	for (;;)
	{
		TOptional<FOutputPakFileEntry*> FromQueue = WriteQueue.Dequeue();
		if (!FromQueue.IsSet())
		{
			if (bFlushed)
			{
				return;
			}
			TRACE_CPUPROFILER_EVENT_SCOPE(WaitingForWork);
			WriteQueueEntryAddedEvent->Wait();
			continue;
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(ProcessFile);

		FOutputPakFileEntry* OutputEntry = FromQueue.GetValue();
		bool bDeleted = OutputEntry->InputPair.bIsDeleteRecord;

		//  Remember the offset but don't serialize it with the entry header.
		FArchive* PakFileHandle = OutputEntry->PakFile->PakFileHandle.Get();
		int64 NewEntryOffset = PakFileHandle->Tell();
		FPakEntryPair& NewEntry = OutputEntry->Entry;

		if (!OutputEntry->EndCompressionTask->IsComplete())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(WaitForCompression);
			OutputEntry->EndCompressionTask->Wait();
		}
		
		const FName& CompressionMethod = OutputEntry->CompressionMethod;
		const FCompressedFileBuffer& CompressedFileBuffer = OutputEntry->GetCompressedBuffer();

		if (!bDeleted)
		{
			const int64& RealFileSize = OutputEntry->RealFileSize;
			const int64& OriginalFileSize = OutputEntry->OriginalFileSize;
			const int32 CompressionBlockSize = CmdLineParameters.CompressionBlockSize;
			NewEntry.Info.CompressionMethodIndex = OutputEntry->PakFile->Info.GetCompressionMethodIndex(CompressionMethod);

			// Account for file system block size, which is a boundary we want to avoid crossing.
			if (!OutputEntry->bIsUAssetUExpPairUExp && // don't split uexp / uasset pairs
				CmdLineParameters.FileSystemBlockSize > 0 &&
				OriginalFileSize != INDEX_NONE &&
				(CmdLineParameters.bAlignFilesLargerThanBlock || RealFileSize <= CmdLineParameters.FileSystemBlockSize) &&
				(NewEntryOffset / CmdLineParameters.FileSystemBlockSize) != ((NewEntryOffset + RealFileSize - 1) / CmdLineParameters.FileSystemBlockSize)) // File crosses a block boundary
			{
				int64 OldOffset = NewEntryOffset;
				NewEntryOffset = AlignArbitrary(NewEntryOffset, CmdLineParameters.FileSystemBlockSize);
				int64 PaddingRequired = NewEntryOffset - OldOffset;

				check(PaddingRequired >= 0);
				check(PaddingRequired < RealFileSize);
				if (PaddingRequired > 0)
				{
					UE_LOGF(LogPakFile, Verbose, "%14llu - %14llu : %14llu padding.", PakFileHandle->Tell(), PakFileHandle->Tell() + PaddingRequired, PaddingRequired);
					while (PaddingRequired > 0)
					{
						int64 AmountToWrite = FMath::Min(PaddingRequired, PaddingBuffer.Num());
						PakFileHandle->Serialize(PaddingBuffer.GetData(), AmountToWrite);
						PaddingRequired -= AmountToWrite;
					}

					check(PakFileHandle->Tell() == NewEntryOffset);
				}
			}

			// Align bulk data
			if (OutputEntry->bIsMappedBulk && CmdLineParameters.AlignForMemoryMapping > 0 && OriginalFileSize != INDEX_NONE && !bDeleted)
			{
				if (!IsAligned(NewEntryOffset + NewEntry.Info.GetSerializedSize(FPakInfo::PakFile_Version_Latest), CmdLineParameters.AlignForMemoryMapping))
				{
					int64 OldOffset = NewEntryOffset;
					NewEntryOffset = AlignArbitrary(NewEntryOffset + NewEntry.Info.GetSerializedSize(FPakInfo::PakFile_Version_Latest), CmdLineParameters.AlignForMemoryMapping) - NewEntry.Info.GetSerializedSize(FPakInfo::PakFile_Version_Latest);
					int64 PaddingRequired = NewEntryOffset - OldOffset;

					check(PaddingRequired > 0);
					check(PaddingBuffer.Num() >= PaddingRequired);

					{
						UE_LOGF(LogPakFile, Verbose, "%14llu - %14llu : %14llu bulk padding.", PakFileHandle->Tell(), PakFileHandle->Tell() + PaddingRequired, PaddingRequired);
						PakFileHandle->Serialize(PaddingBuffer.GetData(), PaddingRequired);
						check(PakFileHandle->Tell() == NewEntryOffset);
					}
				}
			}
		}

		TArray<FFileRegion> CurrentFileRegions;

		int64 SizeToWrite = 0;
		uint8* DataToWrite = nullptr;
		if (bDeleted)
		{
			PrepareDeleteRecordForPak(OutputEntry->PakFile->MountPoint, OutputEntry->InputPair, NewEntry);
			OutputEntry->bCopiedToPak = false;

			// Directly add the new entry to the index, no more work to do
			OutputEntry->PakFile->Index.Add(NewEntry);
		}
		else if (OutputEntry->InputPair.bNeedsCompression && CompressionMethod != NAME_None)
		{
			OutputEntry->bCopiedToPak = PrepareCopyCompressedFileToPak(OutputEntry->PakFile->MountPoint, OutputEntry->PakFile->Info, OutputEntry->InputPair, CompressedFileBuffer, NewEntry, DataToWrite, SizeToWrite, OutputEntry->PakFile->KeyChain);
		}
		else
		{
			OutputEntry->bCopiedToPak = PrepareCopyFileToPak(OutputEntry->PakFile->MountPoint, OutputEntry->InputPair, CompressedFileBuffer, NewEntry, DataToWrite, SizeToWrite, OutputEntry->PakFile->KeyChain, CmdLineParameters.bFileRegions ? &CurrentFileRegions : nullptr);
		}

		int64 TotalSizeToWrite = SizeToWrite + NewEntry.Info.GetSerializedSize(FPakInfo::PakFile_Version_Latest);
		if (OutputEntry->bCopiedToPak)
		{
			if (RequiredPatchPadding > 0 &&
				!(OutputEntry->bIsMappedBulk && CmdLineParameters.AlignForMemoryMapping > 0) // don't wreck the bulk padding with patch padding
				)
			{
				//if the next file is going to cross a patch-block boundary then pad out the current set of files with 0's
				//and align the next file up.
				bool bCrossesBoundary = (NewEntryOffset / RequiredPatchPadding) != ((NewEntryOffset + TotalSizeToWrite - 1) / RequiredPatchPadding);
				bool bPatchPadded = false;
				if (!OutputEntry->bIsUAssetUExpPairUExp) // never patch-pad the uexp of a uasset/uexp pair
				{
					bool bPairProbablyCrossesBoundary = false; // we don't consider compression because we have not compressed the uexp yet.
					if (OutputEntry->bIsUAssetUExpPairUAsset)
					{
						int64 UExpFileSize = OutputEntry->UExpFileInPair->OriginalFileSize / 2; // assume 50% compression
						bPairProbablyCrossesBoundary = (NewEntryOffset / RequiredPatchPadding) != ((NewEntryOffset + TotalSizeToWrite + UExpFileSize - 1) / RequiredPatchPadding);
					}
					if (TotalSizeToWrite >= RequiredPatchPadding || // if it exactly the padding size and by luck does not cross a boundary, we still consider it "over" because it can't be packed with anything else
						bCrossesBoundary || bPairProbablyCrossesBoundary)
					{
						NewEntryOffset = AlignArbitrary(NewEntryOffset, RequiredPatchPadding);
						int64 CurrentLoc = PakFileHandle->Tell();
						int64 PaddingSize = NewEntryOffset - CurrentLoc;
						check(PaddingSize >= 0);
						if (PaddingSize)
						{
							UE_LOGF(LogPakFile, Verbose, "%14llu - %14llu : %14llu patch padding.", PakFileHandle->Tell(), PakFileHandle->Tell() + PaddingSize, PaddingSize);
							check(PaddingSize <= PaddingBuffer.Num());

							//have to pad manually with 0's.  File locations skipped by Seek and never written are uninitialized which would defeat the whole purpose
							//of padding for certain platforms patch diffing systems.
							PakFileHandle->Serialize(PaddingBuffer.GetData(), PaddingSize);
						}
						check(PakFileHandle->Tell() == NewEntryOffset);
						bPatchPadded = true;
					}
				}

				//if the current file is bigger than a patch block then we will always have to pad out the previous files.
				//if there were a large set of contiguous small files behind us then this will be the natural stopping point for a possible pathalogical patching case where growth in the small files causes a cascade 
				//to dirty up all the blocks prior to this one.  If this could happen let's warn about it.
				if (bPatchPadded ||
					OutputEntry->bIsLastFileInPak) // also check the last file, this won't work perfectly if we don't end up adding the last file for some reason
				{
					const uint64 ContiguousGroupedFilePatchWarningThreshhold = 50 * 1024 * 1024;
					if (OutputEntry->PakFile->ContiguousTotalSizeSmallerThanBlockSize > ContiguousGroupedFilePatchWarningThreshhold)
					{
						UE_LOG(LogPakFile, Display, TEXT("%" UINT64_FMT " small files (%" INT64_FMT ") totaling %" UINT64_FMT " contiguous bytes found before first 'large' file.  Changes to any of these files could cause the whole group to be 'dirty' in a per-file binary diff based patching system."), OutputEntry->PakFile->ContiguousFilesSmallerThanBlockSize, RequiredPatchPadding, OutputEntry->PakFile->ContiguousTotalSizeSmallerThanBlockSize);
					}
					OutputEntry->PakFile->ContiguousTotalSizeSmallerThanBlockSize = 0;
					OutputEntry->PakFile->ContiguousFilesSmallerThanBlockSize = 0;
				}
				else
				{
					OutputEntry->PakFile->ContiguousTotalSizeSmallerThanBlockSize += TotalSizeToWrite;
					OutputEntry->PakFile->ContiguousFilesSmallerThanBlockSize++;
				}
			}
			if (OutputEntry->InputPair.bNeedsCompression && CompressionMethod != NAME_None)
			{
				FinalizeCopyCompressedFileToPak(OutputEntry->PakFile->Info, CompressedFileBuffer, NewEntry);
			}

			{
				// track per-compressor stats :
				// note GetCompressionMethodIndex in the pak entry is the index in the Pak file list of compressors
				//	not the same as the index in the command line list

				int32 CompressionMethodIndex;
				if (CompressionFormatsAndNone.Find(CompressionMethod, CompressionMethodIndex))
				{
					OutputEntry->PakFile->Compressor_Stat_Count[CompressionMethodIndex] += 1;
					OutputEntry->PakFile->Compressor_Stat_RawBytes[CompressionMethodIndex] += NewEntry.Info.UncompressedSize;
					OutputEntry->PakFile->Compressor_Stat_CompBytes[CompressionMethodIndex] += NewEntry.Info.Size;
				}
			}

			// Write to file
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(WriteToFile);
				int64 Offset = PakFileHandle->Tell();
				NewEntry.Info.Serialize(*PakFileHandle, FPakInfo::PakFile_Version_Latest);
				int64 PayloadOffset = PakFileHandle->Tell();
				PakFileHandle->Serialize(DataToWrite, SizeToWrite);
				int64 EndOffset = PakFileHandle->Tell();

				if (CmdLineParameters.bFileRegions)
				{
					FFileRegion::AccumulateFileRegions(OutputEntry->PakFile->AllFileRegions, Offset, PayloadOffset, EndOffset, CurrentFileRegions);
				}

				UE_LOGF(LogPakFile, Verbose, "%14llu [header] - %14llu - %14llu : %14llu header+file %ls.", Offset, PayloadOffset, EndOffset, EndOffset - Offset, *NewEntry.Filename);
			}

			// Update offset now and store it in the index (and only in index)
			NewEntry.Info.Offset = NewEntryOffset;
			OutputEntry->PakFile->Index.Add(NewEntry);
			const TCHAR* EncryptedString = TEXT("");

			if (OutputEntry->InputPair.bNeedEncryption)
			{
				OutputEntry->PakFile->TotalRequestedEncryptedFiles++;

				if (OutputEntry->PakFile->KeyChain.GetPrincipalEncryptionKey())
				{
					OutputEntry->PakFile->TotalEncryptedFiles++;
					OutputEntry->PakFile->TotalEncryptedDataSize += SizeToWrite;
					EncryptedString = TEXT("encrypted ");
				}
			}

			if (OutputEntry->InputPair.bNeedsCompression && CompressionMethod != NAME_None)
			{
				OutputEntry->PakFile->TotalCompressedSize += NewEntry.Info.Size;
				OutputEntry->PakFile->TotalUncompressedSize += NewEntry.Info.UncompressedSize;
			}

			if (OutputEntry->InputPair.bNeedRehydration)
			{
				OutputEntry->PakFile->RehydratedCount += OutputEntry->AccessCompressedBuffer().RehydrationCount;
				OutputEntry->PakFile->RehydratedBytes += OutputEntry->AccessCompressedBuffer().RehydrationBytes;
			}
		}
		Retire(OutputEntry);
	}
}

bool FPakWriterContext::Flush()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Flush);
	check(!bFlushed);
	bFlushed = true;

	CompressionQueueEntryAddedEvent->Trigger();
	WriteQueueEntryAddedEvent->Trigger();

	const int64 RequiredPatchPadding = CmdLineParameters.PatchFilePadAlign;

	for (TUniquePtr<FOutputPakFile>& OutputPakFile : OutputPakFiles)
	{
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(WaitForWritesToComplete);
			for (;;)
			{
				bool bCompleted = OutputPakFile->AllEntriesRetiredEvent->Wait(2000);
				if (bCompleted)
				{
					break;
				}
				UE_LOGF(LogPakFile, Display, "Writing entries (%llu/%llu)...", RetiredEntriesCount.Load(), TotalEntriesCount);
			}
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(FinalizePakFile);

		bool bPakFileIsEmpty = OutputPakFile->Index.Num() == 0;

		if (bPakFileIsEmpty)
		{
			UE_LOGF(LogPakFile, Display, "Created empty pak file: %ls", *OutputPakFile->Filename);
		}
		else
		{
			UE_LOGF(LogPakFile, Display, "Created pak file: %ls", *OutputPakFile->Filename);
		}

		for (FOutputPakFileEntry& OutputEntry : OutputPakFile->Entries)
		{
			if (!OutputEntry.bSomeCompressionSucceeded)
			{
				UE_LOGF(LogPakFile, Verbose, "File \"%ls\" did not get small enough from compression, or compression failed.", *OutputEntry.InputPair.Source);
			}
			if (OutputEntry.bCopiedToPak)
			{
				const TCHAR* EncryptedString = TEXT("");

				if (OutputEntry.InputPair.bNeedEncryption)
				{
					if (OutputEntry.PakFile->KeyChain.GetPrincipalEncryptionKey())
					{
						EncryptedString = TEXT("encrypted ");
					}
				}

				if (OutputEntry.InputPair.bNeedsCompression && OutputEntry.CompressionMethod != NAME_None)
				{
					float PercentLess = ((float)OutputEntry.Entry.Info.Size / ((float)OutputEntry.Entry.Info.UncompressedSize / 100.f));
					if (OutputEntry.InputPair.SuggestedOrder < MAX_uint64)
					{
						UE_LOGF(LogPakFile, Verbose, "Added compressed %lsfile \"%ls\", %.2f%% of original size. Compressed with %ls, Size %lld bytes, Original Size %lld bytes (order %llu).", EncryptedString, *OutputEntry.Entry.Filename, PercentLess, *OutputEntry.CompressionMethod.ToString(), OutputEntry.Entry.Info.Size, OutputEntry.Entry.Info.UncompressedSize, OutputEntry.InputPair.SuggestedOrder);
					}
					else
					{
						UE_LOGF(LogPakFile, Verbose, "Added compressed %lsfile \"%ls\", %.2f%% of original size. Compressed with %ls, Size %lld bytes, Original Size %lld bytes (no order given).", EncryptedString, *OutputEntry.Entry.Filename, PercentLess, *OutputEntry.CompressionMethod.ToString(), OutputEntry.Entry.Info.Size, OutputEntry.Entry.Info.UncompressedSize);
					}
				}
				else
				{
					if (OutputEntry.InputPair.SuggestedOrder < MAX_uint64)
					{
						UE_LOGF(LogPakFile, Verbose, "Added %lsfile \"%ls\", %lld bytes (order %llu).", EncryptedString, *OutputEntry.Entry.Filename, OutputEntry.Entry.Info.Size, OutputEntry.InputPair.SuggestedOrder);
					}
					else
					{
						UE_LOGF(LogPakFile, Verbose, "Added %lsfile \"%ls\", %lld bytes (no order given).", EncryptedString, *OutputEntry.Entry.Filename, OutputEntry.Entry.Info.Size);
					}
				}
			}
			else
			{
				if (OutputEntry.InputPair.bIsDeleteRecord)
				{
					UE_LOGF(LogPakFile, Verbose, "Created delete record for file \"%ls\".", *OutputEntry.InputPair.Source);
				}
				else
				{
					UE_LOGF(LogPakFile, Warning, "Missing file \"%ls\" will not be added to PAK file.", *OutputEntry.InputPair.Source);
				}
			}
		}

		if (RequiredPatchPadding > 0)
		{
			for (const FPakEntryPair& Pair : OutputPakFile->Index)
			{
				const FString& EntryFilename = Pair.Filename;
				const FPakEntry& PakEntry = Pair.Info;
				int64 EntrySize = PakEntry.GetSerializedSize(FPakInfo::PakFile_Version_Latest);
				int64 TotalSizeToWrite = PakEntry.Size + EntrySize;
				if (TotalSizeToWrite >= RequiredPatchPadding)
				{
					int64 RealStart = PakEntry.Offset;
					if ((RealStart % RequiredPatchPadding) != 0 &&
						!EntryFilename.EndsWith(TEXT("uexp")) && // these are export sections of larger files and may be packed with uasset/umap and so we don't need a warning here
						!(EntryFilename.EndsWith(TEXT(".m.ubulk")) && CmdLineParameters.AlignForMemoryMapping > 0)) // Bulk padding unaligns patch padding and so we don't need a warning here
					{
						UE_LOG(LogPakFile, Warning, TEXT("File at offset %" INT64_FMT " of size %" INT64_FMT " not aligned to patch size %" INT64_FMT), RealStart, PakEntry.Size, RequiredPatchPadding);
					}
				}
			}
		}


		FPakFooterInfo Footer(*OutputPakFile->Filename, OutputPakFile->MountPoint, OutputPakFile->Info, OutputPakFile->Index);
		Footer.SetEncryptionInfo(OutputPakFile->KeyChain, &OutputPakFile->TotalEncryptedDataSize);
		Footer.SetFileRegionInfo(CmdLineParameters.bFileRegions, OutputPakFile->AllFileRegions);
		WritePakFooter(*OutputPakFile->PakFileHandle, Footer);

		if (CmdLineParameters.bSign)
		{
			TArray<uint8> SignatureData;
			SignatureData.Append(OutputPakFile->Info.IndexHash.Hash, UE_ARRAY_COUNT(FSHAHash::Hash));
			((FSignedArchiveWriter*)OutputPakFile->PakFileHandle.Get())->SetSignatureData(SignatureData);
		}

		uint64 TotalPakFileSize = OutputPakFile->PakFileHandle->TotalSize();
		OutputPakFile->PakFileHandle->Close();
		OutputPakFile->PakFileHandle.Reset();

		if (CmdLineParameters.bFileRegions)
		{
			FFileRegion::SerializeFileRegions(*OutputPakFile->PakFileRegionsHandle.Get(), OutputPakFile->AllFileRegions);
			OutputPakFile->PakFileRegionsHandle->Close();
			OutputPakFile->PakFileRegionsHandle.Reset();
		}

		if (bPakFileIsEmpty == false)
		{
			// log per-compressor stats :
			for (int32 MethodIndex = 0; MethodIndex < CompressionFormatsAndNone.Num(); MethodIndex++)
			{
				if (OutputPakFile->Compressor_Stat_Count[MethodIndex])
				{
					FName CompressionMethod = CompressionFormatsAndNone[MethodIndex];
					UE_LOGF(LogPakFile, Display, "CompressionFormat %d [%ls] : %d files, %lld -> %lld bytes", MethodIndex, *(CompressionMethod.ToString()),
						OutputPakFile->Compressor_Stat_Count[MethodIndex],
						OutputPakFile->Compressor_Stat_RawBytes[MethodIndex],
						OutputPakFile->Compressor_Stat_CompBytes[MethodIndex]
					);
				}
			}

			UE_LOGF(LogPakFile, Display, "Added %d files, %lld bytes total", OutputPakFile->Index.Num(), TotalPakFileSize);
			UE_LOG(LogPakFile, Display, TEXT("PrimaryIndex size: %" INT64_FMT " bytes"), Footer.PrimaryIndexSize);
			UE_LOG(LogPakFile, Display, TEXT("PathHashIndex size: %" INT64_FMT " bytes"), Footer.PathHashIndexSize);
			UE_LOG(LogPakFile, Display, TEXT("FullDirectoryIndex size: %" INT64_FMT " bytes"), Footer.FullDirectoryIndexSize);
			if (OutputPakFile->TotalUncompressedSize)
			{
				float PercentLess = ((float)OutputPakFile->TotalCompressedSize / ((float)OutputPakFile->TotalUncompressedSize / 100.f));
				UE_LOG(LogPakFile, Display, TEXT("Compression summary: %.2f%% of original size. Compressed Size %" UINT64_FMT " bytes, Original Size %" UINT64_FMT " bytes. "), PercentLess, OutputPakFile->TotalCompressedSize, OutputPakFile->TotalUncompressedSize);
			}

			if (OutputPakFile->RehydratedCount > 0)
			{
				UE_LOGF(LogPakFile, Display, "Asset Rehydration");

				UE_LOG(LogPakFile, Display, TEXT("  Rehydrated: %" UINT64_FMT " payloads"), OutputPakFile->RehydratedCount);
				UE_LOG(LogPakFile, Display, TEXT("  Rehydrated: %" UINT64_FMT " bytes (%.2fMB)"), OutputPakFile->RehydratedBytes, (float)OutputPakFile->RehydratedBytes / 1024.0f / 1024.0f);
			}

			if (OutputPakFile->TotalEncryptedDataSize)
			{
				UE_LOGF(LogPakFile, Display, "Encryption - ENABLED");
				UE_LOG(LogPakFile, Display, TEXT("  Files: %" UINT64_FMT), OutputPakFile->TotalEncryptedFiles);

				if (OutputPakFile->Info.bEncryptedIndex)
				{
					UE_LOG(LogPakFile, Display, TEXT("  Index: Encrypted (%" INT64_FMT " bytes, %.2fMB)"), OutputPakFile->Info.IndexSize, (float)OutputPakFile->Info.IndexSize / 1024.0f / 1024.0f);
				}
				else
				{
					UE_LOGF(LogPakFile, Display, "  Index: Unencrypted");
				}


				UE_LOG(LogPakFile, Display, TEXT("  Total: %" UINT64_FMT " bytes (%.2fMB)"), OutputPakFile->TotalEncryptedDataSize, (float)OutputPakFile->TotalEncryptedDataSize / 1024.0f / 1024.0f);
			}
			else
			{
				UE_LOGF(LogPakFile, Display, "Encryption - DISABLED");
			}

			if (OutputPakFile->TotalEncryptedFiles < OutputPakFile->TotalRequestedEncryptedFiles)
			{
				UE_LOG(LogPakFile, Display, TEXT("%" UINT64_FMT " files requested encryption, but no AES key was supplied! Encryption was skipped for these files"), OutputPakFile->TotalRequestedEncryptedFiles);
			}
			UE_LOGF(LogPakFile, Display, "");
		}
	}

	WriterThread.Wait();
	CompressionThread.Wait();
	UE_LOGF(LogPakFile, Display, "Writer and Compression Threads exited.");

	if (TotalFilesWithPoorForcedCompression > 0)
	{
		UE_LOG(LogPakFile, Display, TEXT("Num files forcibly compressed due to -forcecompress option: %" INT64_FMT ", using %" INT64_FMT "bytes extra"), (int64)TotalFilesWithPoorForcedCompression, (int64)TotalExtraMemoryForPoorForcedCompression);
	}

#if DETAILED_UNREALPAK_TIMING
	UE_LOGF(LogPakFile, Display, "Detailed timing stats");
	UE_LOGF(LogPakFile, Display, "Compression time: %lf", ((double)GCompressionTime) * FPlatformTime::GetSecondsPerCycle());
	UE_LOGF(LogPakFile, Display, "DDC Hits: %d", GDDCHits);
	UE_LOGF(LogPakFile, Display, "DDC Misses: %d", GDDCMisses);
	UE_LOGF(LogPakFile, Display, "DDC Sync Read Time: %lf", ((double)GDDCSyncReadTime) * FPlatformTime::GetSecondsPerCycle());
	UE_LOGF(LogPakFile, Display, "DDC Sync Write Time: %lf", ((double)GDDCSyncWriteTime) * FPlatformTime::GetSecondsPerCycle());
#endif

	if (CmdLineParameters.CsvPath.Len() > 0)
	{
		int64 SizeFilter = 0;
		bool bIncludeDeleted = true;
		bool bExtractToMountPoint = true;

		bool bPerPakCsvFiles = FPaths::DirectoryExists(CmdLineParameters.CsvPath);
		FString CsvFilename;
		if (!bPerPakCsvFiles)
		{
			// When CsvPath is a filename append .pak.csv to create a unique single csv for all pak files,
			// different from the unique single .utoc.csv for all container files.
			CsvFilename = CmdLineParameters.CsvPath + TEXT(".pak.csv");
		}
		bool bAppendFile = false;
		for (TUniquePtr<FOutputPakFile>& OutputPakFile : OutputPakFiles)
		{
			if (bPerPakCsvFiles)
			{
				// When CsvPath is a dir, then create one unique .pak.csv per pak file
				CsvFilename = CmdLineParameters.CsvPath / FPaths::GetCleanFilename(OutputPakFile->Filename) + TEXT(".csv");
			}
			ListFilesInPak(
				*OutputPakFile->Filename,
				SizeFilter,
				bIncludeDeleted,
				CsvFilename,
				bExtractToMountPoint,
				OutputPakFile->KeyChain,
				bAppendFile,
				false);
			if (!bPerPakCsvFiles)
			{
				bAppendFile = true;
			}
		}
	}
	return true;
}

bool CreatePakFile(const TCHAR* Filename, const TArray<FPakInputPair>& FilesToAdd, const FPakCommandLineParameters& CmdLineParameters, const FKeyChain& InKeyChain)
{
	const double StartTime = FPlatformTime::Seconds();

	// Create Pak
	FPakWriterContext PakWriterContext;
	if (!PakWriterContext.Initialize(CmdLineParameters))
	{
		return false;
	}
	
	if (!PakWriterContext.AddPakFile(Filename, FilesToAdd, InKeyChain))
	{
		PakWriterContext.Flush();
		return false;
	}

	return PakWriterContext.Flush();
}

void WritePakFooter(FArchive& PakHandle, FPakFooterInfo& Footer)
{
	// Create the second copy of the FPakEntries stored in the Index at the end of the PakFile
	// FPakEntries in the Index are stored as compacted bytes in EncodedPakEntries if possible, or in uncompacted NonEncodableEntries if not.
	// At the same time, create the two Indexes that map to the encoded-or-not given FPakEntry.  The runtime will only load one of these, depending
	// on whether it needs to be able to do DirectorySearches.
	auto FinalizeIndexBlockSize = [&Footer](TArray<uint8>& IndexData)
	{
		if (Footer.Info.bEncryptedIndex)
		{
			int32 OriginalSize = IndexData.Num();
			int32 AlignedSize = Align(OriginalSize, FAES::AESBlockSize);

			for (int32 PaddingIndex = IndexData.Num(); PaddingIndex < AlignedSize; ++PaddingIndex)
			{
				uint8 Byte = IndexData[PaddingIndex % OriginalSize];
				IndexData.Add(Byte);
			}
		}
	};
	auto FinalizeIndexBlock = [&PakHandle, &Footer, &FinalizeIndexBlockSize](TArray<uint8>& IndexData, FSHAHash& OutHash)
	{
		FinalizeIndexBlockSize(IndexData);

		FSHA1::HashBuffer(IndexData.GetData(), IndexData.Num(), OutHash.Hash);

		if (Footer.Info.bEncryptedIndex)
		{
			check(Footer.KeyChain && Footer.KeyChain->GetPrincipalEncryptionKey() && Footer.TotalEncryptedDataSize);
			FAES::EncryptData(IndexData.GetData(), IndexData.Num(), Footer.KeyChain->GetPrincipalEncryptionKey()->Key);
			*Footer.TotalEncryptedDataSize += IndexData.Num();
		}
	};

	TArray<uint8> EncodedPakEntries;
	int32 NumEncodedEntries = 0;
	int32 NumDeletedEntries = 0;
	TArray<FPakEntry> NonEncodableEntries;

	FPakFile::FDirectoryIndex DirectoryIndex;
	FPakFile::FPathHashIndex PathHashIndex;
	TMap<uint64, FString> CollisionDetection; // Currently detecting Collisions only within the files stored into a single Pak.  TODO: Create separate job to detect collisions over all files in the export.
	uint64 PathHashSeed;
	int32 NextIndex = 0;
	auto ReadNextEntry = [&Footer, &NextIndex]() -> FPakEntryPair&
	{
		return Footer.Index[NextIndex++];
	};

	FPakFile::EncodePakEntriesIntoIndex(Footer.Index.Num(), ReadNextEntry, Footer.Filename,
		Footer.Info, Footer.MountPoint, NumEncodedEntries, NumDeletedEntries, &PathHashSeed,
		&DirectoryIndex, &PathHashIndex, EncodedPakEntries, NonEncodableEntries, &CollisionDetection,
		FPakInfo::PakFile_Version_Latest);
	VerifyIndexesMatch(Footer.Index, DirectoryIndex, PathHashIndex, PathHashSeed, Footer.MountPoint,
		EncodedPakEntries, NonEncodableEntries, NumEncodedEntries, NumDeletedEntries, Footer.Info);

	// We write one PrimaryIndex and two SecondaryIndexes to the Pak File
	// PrimaryIndex
	//		Common scalar data such as MountPoint
	//		PresenceBit and Offset,Size,Hash for the SecondaryIndexes
	//		PakEntries (Encoded and NonEncoded)
	// SecondaryIndex PathHashIndex: used by default in shipped versions of games.  Uses less memory, but does not provide access to all filenames.
	//		TMap from hash of FilePath to FPakEntryLocation
	//		Pruned DirectoryIndex, containing only the FilePaths that were requested kept by allow list config variables
	// SecondaryIndex FullDirectoryIndex: used for developer tools and for titles that opt out of PathHashIndex because they need access to all filenames.
	//		TMap from DirectoryPath to FDirectory, which itself is a TMap from CleanFileName to FPakEntryLocation
	// Each Index is separately encrypted and hashed.  Runtime consumer such as the tools or the client game will only load one of these off of disk (unless runtime verification is turned on).

	// Create the pruned DirectoryIndex for use in the Primary Index
	TMap<FString, FPakDirectory> PrunedDirectoryIndex;
	FPakFile::PruneDirectoryIndex(DirectoryIndex, &PrunedDirectoryIndex, Footer.MountPoint);

	bool bWritePathHashIndex = FPakFile::IsPakWritePathHashIndex();
	bool bWriteFullDirectoryIndex = FPakFile::IsPakWriteFullDirectoryIndex();
	checkf(bWritePathHashIndex || bWriteFullDirectoryIndex, TEXT("At least one of Engine:[Pak]:WritePathHashIndex and Engine:[Pak]:WriteFullDirectoryIndex must be true"));

	TArray<uint8> PrimaryIndexData;
	TArray<uint8> PathHashIndexData;
	TArray<uint8> FullDirectoryIndexData;
	Footer.Info.IndexOffset = PakHandle.Tell();
	// Write PrimaryIndex bytes
	{
		FMemoryWriter PrimaryIndexWriter(PrimaryIndexData);
		PrimaryIndexWriter.SetByteSwapping(PakHandle.ForceByteSwapping());

		PrimaryIndexWriter << const_cast<FString&>(Footer.MountPoint);
		int32 NumEntries = Footer.Index.Num();
		PrimaryIndexWriter << NumEntries;
		PrimaryIndexWriter << PathHashSeed;

		FSecondaryIndexWriter PathHashIndexWriter(PathHashIndexData, bWritePathHashIndex, PrimaryIndexData, PrimaryIndexWriter);
		PathHashIndexWriter.WritePlaceholderToPrimary();

		FSecondaryIndexWriter FullDirectoryIndexWriter(FullDirectoryIndexData, bWriteFullDirectoryIndex, PrimaryIndexData, PrimaryIndexWriter);
		FullDirectoryIndexWriter.WritePlaceholderToPrimary();

		PrimaryIndexWriter << EncodedPakEntries;
		int32 NonEncodableEntriesNum = NonEncodableEntries.Num();
		PrimaryIndexWriter << NonEncodableEntriesNum;
		for (FPakEntry& PakEntry : NonEncodableEntries)
		{
			PakEntry.Serialize(PrimaryIndexWriter, FPakInfo::PakFile_Version_Latest);
		}

		// Finalize the size of the PrimaryIndex (it may change due to alignment padding) because we need the size to know the offset of the SecondaryIndexes which come after it in the PakFile.
		// Do not encrypt and hash it yet, because we still need to replace placeholder data in it for the Offset,Size,Hash of each SecondaryIndex
		FinalizeIndexBlockSize(PrimaryIndexData);

		// Write PathHashIndex bytes
		if (bWritePathHashIndex)
		{
			{
				FMemoryWriter& SecondaryWriter = PathHashIndexWriter.GetSecondaryWriter();
				SecondaryWriter << PathHashIndex;
				FPakFile::SaveIndexInternal_DirectoryIndex(SecondaryWriter, PrunedDirectoryIndex);
			}
			PathHashIndexWriter.FinalizeAndRecordOffset(Footer.Info.IndexOffset + PrimaryIndexData.Num(), FinalizeIndexBlock);
		}

		// Write FullDirectoryIndex bytes
		if (bWriteFullDirectoryIndex)
		{
			{
				FMemoryWriter& SecondaryWriter = FullDirectoryIndexWriter.GetSecondaryWriter();
				FPakFile::SaveIndexInternal_DirectoryIndex(SecondaryWriter, DirectoryIndex);
			}
			FullDirectoryIndexWriter.FinalizeAndRecordOffset(Footer.Info.IndexOffset + PrimaryIndexData.Num() + PathHashIndexData.Num(), FinalizeIndexBlock);
		}

		// Encrypt and Hash the PrimaryIndex now that we have filled in the SecondaryIndex information
		FinalizeIndexBlock(PrimaryIndexData, Footer.Info.IndexHash);
	}

	// Write the bytes for each Index into the PakFile
	Footer.Info.IndexSize = PrimaryIndexData.Num();
	PakHandle.Serialize(PrimaryIndexData.GetData(), PrimaryIndexData.Num());
	if (bWritePathHashIndex)
	{
		PakHandle.Serialize(PathHashIndexData.GetData(), PathHashIndexData.Num());
	}
	if (bWriteFullDirectoryIndex)
	{
		PakHandle.Serialize(FullDirectoryIndexData.GetData(), FullDirectoryIndexData.Num());
	}

	// Save the FPakInfo, which has offset, size, and hash value for the PrimaryIndex, at the end of the PakFile
	Footer.Info.Serialize(PakHandle, FPakInfo::PakFile_Version_Latest);

	if (Footer.bFileRegions)
	{
		check(Footer.AllFileRegions);
		// Add a final region to include the headers / data at the end of the .pak, after the last file payload.
		FFileRegion::AccumulateFileRegions(*Footer.AllFileRegions, Footer.Info.IndexOffset, Footer.Info.IndexOffset, PakHandle.Tell(), {});
	}

	Footer.PrimaryIndexSize = PrimaryIndexData.Num();
	Footer.PathHashIndexSize = PathHashIndexData.Num();
	Footer.FullDirectoryIndexSize = FullDirectoryIndexData.Num();
}

bool TestPakFile(const TCHAR* Filename, bool TestHashes)
{	
	TRefCountPtr<FPakFile> PakFile = MakeRefCount<FPakFile>(&FPlatformFileManager::Get().GetPlatformFile(), Filename, /*bIsSigned=*/ false);
	if (PakFile->IsValid())
	{
		return TestHashes ? PakFile->Check() : true;
	}
	else
	{
		UE_LOGF(LogPakFile, Error, "Unable to open pak file \"%ls\".", Filename);
		return false;
	}
}

bool ListFilesInPak(const TCHAR * InPakFilename, int64 SizeFilter, bool bIncludeDeleted, const FString& CSVFilename, bool bExtractToMountPoint, const FKeyChain& InKeyChain, bool bAppendFile, bool bIncludeFooter)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ListFilesInPak);
	IPlatformFile* LowerLevelPlatformFile = &FPlatformFileManager::Get().GetPlatformFile();
	TRefCountPtr<FPakFile> PakFilePtr = MakeRefCount<FPakFile>(LowerLevelPlatformFile, InPakFilename, /*bIsSigned=*/ false);
	FPakFile& PakFile = *PakFilePtr;
	int32 FileCount = 0;
	int64 FileSize = 0;
	int64 FilteredSize = 0;

	if (PakFile.IsValid())
	{
		UE_LOGF(LogPakFile, Display, "Listing %ls with mount point \"%ls\"",
			*FPaths::GetCleanFilename(InPakFilename), *PakFile.GetMountPoint());

		TArray<TPair<TOptional<FString>, FPakEntry>> Records;

		for (FPakFile::FPakEntryIterator It(PakFile,bIncludeDeleted); It; ++It)
		{
			const FString* Filename = It.TryGetFilename();
			Records.Add({ Filename ? TOptional<FString>(*Filename) : TOptional<FString>(), It.Info()});
		}

		struct FOffsetSort
		{
			FORCEINLINE bool operator()(const TPair<TOptional<FString>, FPakEntry>& A,
				const TPair<TOptional<FString>, FPakEntry>& B) const
			{
				return A.Value.Offset < B.Value.Offset;
			}
		};

		Records.Sort(FOffsetSort());

		const FString MountPoint = bExtractToMountPoint ? PakFile.GetMountPoint() : TEXT("");
		FileCount = Records.Num();

		// The Hashes are not stored in the FPakEntries stored in the index, but they are stored for the FPakEntries stored before each payload.
		// Read the hashes out of the payload
		TArray<uint8> HashesBuffer;
		HashesBuffer.SetNum(FileCount * sizeof(FPakEntry::Hash));
		uint8* Hashes = HashesBuffer.GetData();
		int32 EntryIndex = 0;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ReadHashFromPayload);

			FSharedPakReader SharedReader = PakFile.GetSharedReader(nullptr);

			for (const TPair<TOptional<FString>, FPakEntry>& It : Records)
			{
				SharedReader->Seek(It.Value.Offset);

				FPakEntry SerializedPakEntry;
				SerializedPakEntry.Serialize(SharedReader.GetArchive(), PakFile.GetInfo().Version);
				FMemory::Memcpy(Hashes + (EntryIndex++) * sizeof(FPakEntry::Hash), &SerializedPakEntry.Hash, sizeof(SerializedPakEntry.Hash));
			}
		}

		if (CSVFilename.Len() > 0)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(WriteCsv);
			uint32 WriteFlags = bAppendFile ? FILEWRITE_Append : 0;
			TUniquePtr<FArchive> OutputArchive(IFileManager::Get().CreateFileWriter(*CSVFilename, WriteFlags));
			if (!OutputArchive.IsValid())
			{
				UE_LOGF(LogPakFile, Display, "Failed to save CSV file %ls", *CSVFilename);
				return false;
			}

			if (!bAppendFile)
			{
				OutputArchive->Logf(TEXT("Filename, Offset, Size, Hash, Deleted, Compressed, CompressionMethod"));
			}
			EntryIndex = 0;
			for (const TPair<TOptional<FString>, FPakEntry>& It : Records)
			{
				const FPakEntry& Entry = It.Value;
				uint8* Hash = Hashes + (EntryIndex++)*sizeof(FPakEntry::Hash);

				bool bWasCompressed = Entry.CompressionMethodIndex != 0;
				OutputArchive->Logf(
					TEXT("%s%s, %lld, %lld, %s, %s, %s, %d"),
					*MountPoint, It.Key ? **It.Key : TEXT("<FileNamesNotLoaded>"),
					Entry.Offset, Entry.Size,
					*BytesToHex(Hash, sizeof(FPakEntry::Hash)),
					Entry.IsDeleteRecord() ? TEXT("true") : TEXT("false"),
					bWasCompressed ? TEXT("true") : TEXT("false"),
					Entry.CompressionMethodIndex);
			}
			if (bIncludeFooter)
			{
				// If desired, emit a pseudoentry for our metadata. This is all stored at the end of the file
				// so rather than ask the pak file anything we just assign it to everything past the end of the 
				// records.
				int64 RecordEndOffset = 0;
				if (Records.Num())
				{
					const FPakEntry& LastEntry = Records[Records.Num()-1].Value;
					RecordEndOffset = LastEntry.Offset + LastEntry.Size;
				}
				OutputArchive->Logf(
					TEXT("<Footer>, %lld, %lld, 0, false, false, 0"),
					RecordEndOffset, PakFile.TotalSize() - RecordEndOffset);
			}

			OutputArchive->Flush();
			UE_LOGF(LogPakFile, Display, "Saved CSV file to %ls", *CSVFilename);
			return true;
		}

		TSet<int32> InspectChunks;
		FString InspectChunkString;
		FParse::Value(FCommandLine::Get(), TEXT("InspectChunk="), InspectChunkString, false);
		TArray<FString> InspectChunkRanges;
		if (InspectChunkString.TrimStartAndEnd().ParseIntoArray(InspectChunkRanges, TEXT(",")))
		{
			for (const FString& InspectChunkRangeString : InspectChunkRanges)
			{
				TArray<FString> RangeLimits;
				if (InspectChunkRangeString.TrimStartAndEnd().ParseIntoArray(RangeLimits, TEXT("-")))
				{
					if (RangeLimits.Num() == 1)
					{
						int32 Chunk = -1;
						LexFromString(Chunk, *InspectChunkRangeString);
						if (Chunk != -1)
						{
							InspectChunks.Add(Chunk);
						}
					}
					else if (RangeLimits.Num() == 2)
					{
						int32 FirstChunk = -1;
						int32 LastChunk = -1;
						LexFromString(FirstChunk, *RangeLimits[0]);
						LexFromString(LastChunk, *RangeLimits[1]);
						if (FirstChunk != -1 && LastChunk != -1)
						{
							for (int32 Chunk = FirstChunk; Chunk <= LastChunk; ++Chunk)
							{
								InspectChunks.Add(Chunk);
							}
						}
					}
					else
					{
						UE_LOGF(LogPakFile, Error, "Error parsing inspect chunk range '%ls'", *InspectChunkRangeString);
					}
				}
			}
		}
		EntryIndex = 0;
		for (const TPair<TOptional<FString>, FPakEntry>& It : Records)
		{
			const FPakEntry& Entry = It.Value;
			uint8* Hash = Hashes + (EntryIndex++) * sizeof(FPakEntry::Hash);
			if (Entry.Size >= SizeFilter)
			{
				if (InspectChunkRanges.Num() > 0)
				{
					int32 FirstChunk = IntCastChecked<int32>(Entry.Offset) / (64 * 1024);
					int32 LastChunk = IntCastChecked<int32>(Entry.Offset + Entry.Size) / (64 * 1024);

					for (int32 Chunk = FirstChunk; Chunk <= LastChunk; ++Chunk)
					{
						if (InspectChunks.Contains(Chunk))
						{
							UE_LOG(LogPakFile, Display, TEXT("[%d - %d] \"%s%s\" offset: %" INT64_FMT ", size: %" INT64_FMT " bytes, sha1: %s, compression: %s."),
								FirstChunk, LastChunk, *MountPoint,
								It.Key ? **It.Key : TEXT("<FileNamesNotLoaded"), Entry.Offset, Entry.Size,
								*BytesToHex(Hash, sizeof(FPakEntry::Hash)),
								*PakFile.GetInfo().GetCompressionMethod(Entry.CompressionMethodIndex).ToString());
							break;
						}
					}
				}
				else
				{
					UE_LOG(LogPakFile, Display, TEXT("\"%s%s\" offset: %" INT64_FMT ", size: %" INT64_FMT " bytes, sha1: %s, compression: %s."),
						*MountPoint, It.Key ? **It.Key : TEXT("<FileNamesNotLoaded"),
						Entry.Offset, Entry.Size, *BytesToHex(Hash, sizeof(FPakEntry::Hash)),
						*PakFile.GetInfo().GetCompressionMethod(Entry.CompressionMethodIndex).ToString());
				}
			}
			FileSize += Entry.Size;
		}
		UE_LOGF(LogPakFile, Display, "%d files (%lld bytes), (%lld filtered bytes).", FileCount, FileSize, FilteredSize);

		return true;
	}
	else
	{
		if (PakFile.GetInfo().Magic != 0 && PakFile.GetInfo().EncryptionKeyGuid.IsValid() && !InKeyChain.GetEncryptionKeys().Contains(PakFile.GetInfo().EncryptionKeyGuid))
		{
			UE_LOGF(LogPakFile, Fatal, "Missing encryption key %ls for pak file \"%ls\".", *PakFile.GetInfo().EncryptionKeyGuid.ToString(), InPakFilename);
		}
		else if (LegacyListIoStoreContainer(InPakFilename, SizeFilter, CSVFilename, InKeyChain))
		{
			return true;
		}
		else
		{
			UE_LOGF(LogPakFile, Fatal, "Unable to open pak file \"%ls\".", InPakFilename);
		}

		return false;
	}
}

int32 GetPakPriorityFromFilename( const FString& PakFilename )
{
	// Parse the pak file index, the base pak file is index -1
	int32 PakPriority = -1;
	if (PakFilename.EndsWith("_P.pak"))
	{
		FString PakIndexFromFilename = PakFilename.LeftChop(6);
		int32 PakIndexStart = INDEX_NONE;
		PakIndexFromFilename.FindLastChar('_', PakIndexStart);
		if (PakIndexStart != INDEX_NONE)
		{
			PakIndexFromFilename.RightChopInline(PakIndexStart + 1, EAllowShrinking::No);
			if (PakIndexFromFilename.IsNumeric())
			{
				PakPriority = FCString::Atoi(*PakIndexFromFilename);
			}
		}
	}

	return PakPriority;
}

int32 GetPakChunkIndexFromFilename( const FString& PakFilePath )
{
	FStringView PakChunkPrefix(FGenericPlatformMisc::GetPakFilenamePrefix());
	const int32 PakChunkPrefixLength = 8;//FCString::Strlen(PakChunkPrefix);

	int32 PakChunkIndex = -1;
	FString PakFilename = FPaths::GetCleanFilename(PakFilePath);
	if (PakFilename.StartsWith(PakChunkPrefix))
	{
		int32 ChunkIndexStart = INDEX_NONE;
		if( PakFilename.FindChar(TEXT('-'), ChunkIndexStart ) )
		{
			FString PakChunkFromFilename = PakFilename.Mid( PakChunkPrefixLength, ChunkIndexStart - PakChunkPrefixLength );
			if( PakChunkFromFilename.IsNumeric() )
			{
				PakChunkIndex = FCString::Atoi(*PakChunkFromFilename);
			}
		}
	}

	return PakChunkIndex;
}

bool AuditPakFiles( const FString& InputPath, bool bOnlyDeleted, const FString& CSVFilename, const FPakOrderMap& OrderMap, bool bSortByOrdering )
{
	//collect all pak files
	FString PakFileDirectory;
	TArray<FString> PakFileList;
	if (FPaths::DirectoryExists(InputPath))
	{
		//InputPath is a directory
		IFileManager::Get().FindFiles(PakFileList, *InputPath, TEXT(".pak") );
		PakFileDirectory = InputPath;
	}
	else
	{
		//InputPath is a search wildcard (or a directory that doesn't exist...)
		IFileManager::Get().FindFiles(PakFileList, *InputPath, true, false);
		PakFileDirectory = FPaths::GetPath(InputPath);
	}
	if (PakFileList.Num() == 0)
	{
		UE_LOGF(LogPakFile, Error, "No pak files found searching \"%ls\".", *InputPath);
		return false;
	}
	
	struct FFilePakRevision
	{
		FString PakFilename;
		int32 PakPriority;
		int64 Size;
	};
	TMap<FString, FFilePakRevision> FileRevisions;
	TMap<FString, FFilePakRevision> DeletedRevisions;
	TMap<FString, FString> PakFilenameToPatchDotChunk;
	int32 HighestPakPriority = -1;

	//build lookup tables for the newest revision of all files and all deleted files
	for (int32 PakFileIndex = 0; PakFileIndex < PakFileList.Num(); PakFileIndex++)
	{
		FString PakFilename = PakFileDirectory + "\\" + PakFileList[PakFileIndex];
		int32 PakPriority = GetPakPriorityFromFilename(PakFilename);
		HighestPakPriority = FMath::Max( HighestPakPriority, PakPriority );

		TRefCountPtr<FPakFile> PakFilePtr = MakeRefCount<FPakFile>(&FPlatformFileManager::Get().GetPlatformFile(), *PakFilename, /*bIsSigned=*/ false);
		FPakFile& PakFile = *PakFilePtr;
		if (PakFile.IsValid())
		{
			FString PakMountPoint = PakFile.GetMountPoint().Replace(TEXT("../../../"), TEXT(""));

			const bool bIncludeDeleted = true;
			if (!PakFile.HasFilenames())
			{
				UE_LOGF(LogPakFile, Error, "Pakfiles were loaded without Filenames, cannot audit.");
				return false;
			}

			for (FPakFile::FPakEntryIterator It(PakFile,bIncludeDeleted); It; ++It)
			{
				FString AssetName = PakMountPoint;
				if (!AssetName.IsEmpty() && !AssetName.EndsWith("/"))
				{
					AssetName += "/";
				}
				AssetName += *It.TryGetFilename();

				FFilePakRevision Revision;
				Revision.PakFilename = PakFileList[PakFileIndex];
				Revision.PakPriority = PakPriority;
				Revision.Size = It.Info().Size;

				//add or update the entry for the appropriate revision, depending on whether this is a delete record or not
				TMap<FString, FFilePakRevision>& AppropriateRevisions = (It.Info().IsDeleteRecord()) ? DeletedRevisions : FileRevisions;
				if (!AppropriateRevisions.Contains(AssetName))
				{
					AppropriateRevisions.Add(AssetName, Revision);
				}
				else if (AppropriateRevisions[AssetName].PakPriority < Revision.PakPriority)
				{
					AppropriateRevisions[AssetName] = Revision;
				}
			}


			//build "patch.chunk" string
			FString PatchDotChunk;
			PatchDotChunk += FString::Printf( TEXT("%d."), PakPriority+1 );
			int32 ChunkIndex = GetPakChunkIndexFromFilename( PakFilename );
			if( ChunkIndex != -1 )
			{
				PatchDotChunk += FString::Printf( TEXT("%d"), ChunkIndex );
			}
			PakFilenameToPatchDotChunk.Add( PakFileList[PakFileIndex], PatchDotChunk );
		}
		else
		{
			UE_LOGF(LogPakFile, Error, "Unable to open pak file \"%ls\".", *PakFilename);
			return false;
		}
	}

	bool bHasOpenOrder = (OrderMap.Num() > 0);

	//open CSV file, if requested
	FArchive* CSVFileWriter = nullptr;
	if( !CSVFilename.IsEmpty() )
	{
		CSVFileWriter = IFileManager::Get().CreateFileWriter(*CSVFilename);
		if (CSVFileWriter == nullptr)
		{
			UE_LOGF(LogPakFile, Error, "Unable to open csv file \"%ls\".", *CSVFilename);
			return false;
		}
	}

	//helper lambda for writing line depending on whether there's a CSV file or not
	auto WriteCSVLine = [CSVFileWriter]( const FString& Text )
	{
		if( CSVFileWriter )
		{
			CSVFileWriter->Logf( TEXT("%s"), *Text );
		}
		else
		{
			UE_LOGF(LogPakFile, Display, "%ls", *Text );
		}
	};

	//cache open order for faster lookup
	TMap<FString,uint64> CachedOpenOrder;
	if( bHasOpenOrder )
	{
		UE_LOGF(LogPakFile, Display, "Checking open order data" );
		for (auto Itr : FileRevisions)
		{
			const FString& AssetPath = Itr.Key;
			FString OpenOrderAssetName = FString::Printf( TEXT("../../../%s"), *AssetPath );
			FPaths::NormalizeFilename(OpenOrderAssetName);
			OpenOrderAssetName.ToLowerInline();

			uint64 OrderIndex = OrderMap.GetFileOrder(OpenOrderAssetName, false);
			if (OrderIndex != MAX_uint64)
			{
				CachedOpenOrder.Add( AssetPath, OrderIndex );
			}
		}
	}

	//helper lambda to look up cached open order
	auto FindOpenOrder = [&]( const FString& AssetPath )
	{
		if( const uint64* OrderIndexPtr = CachedOpenOrder.Find( AssetPath ) )
		{
			return (*OrderIndexPtr);
		}

		return uint64(UINT64_MAX);
	};

	//log every file, sorted alphabetically
	if( bSortByOrdering && bHasOpenOrder )
	{
		UE_LOGF(LogPakFile, Display, "Sorting pak audit data by open order" );
		FileRevisions.KeySort([&]( const FString& A, const FString& B )
		{
			return FindOpenOrder(A) < FindOpenOrder(B);
		});
		DeletedRevisions.KeySort([&]( const FString& A, const FString& B )
		{
			return FindOpenOrder(A) < FindOpenOrder(B);
		});
	}
	else
	{
		UE_LOGF(LogPakFile, Display, "Sorting pak audit data by name" );
		FileRevisions.KeySort([]( const FString& A, const FString& B )
		{
			return A.Compare(B, ESearchCase::IgnoreCase) < 0;
		});
		DeletedRevisions.KeySort([]( const FString& A, const FString& B )
		{
			return A.Compare(B, ESearchCase::IgnoreCase) < 0;
		});
	}

	FString PreviousPatchDotChunk;
	int NumSeeks = 0;
	int NumReads = 0;

	UE_CLOGF((CSVFileWriter!=nullptr),LogPakFile, Display, "Writing pak audit CSV file %ls...", *CSVFilename );
	WriteCSVLine( TEXT("AssetName,State,Pak,Prev.Pak,Rev,Prev.Rev,Size,AssetPath,Patch.Chunk,OpenOrder" ) );
	for (auto Itr : FileRevisions)
	{
		const FString& AssetPath = Itr.Key;
		const FString AssetName = FPaths::GetCleanFilename(AssetPath);
		const FFilePakRevision* DeletedRevision = DeletedRevisions.Find(AssetPath);

		//look up the open order for this file
		FString OpenOrderText = "";
		uint64 OpenOrder = FindOpenOrder(AssetPath);
		if( OpenOrder != UINT64_MAX )
		{
			OpenOrderText = FString::Printf( TEXT("%llu"), OpenOrder );
		}

		//lookup patch.chunk value
		FString PatchDotChunk = "";
		if( const FString* PatchDotChunkPtr = PakFilenameToPatchDotChunk.Find(Itr.Value.PakFilename) )
		{
			PatchDotChunk = *PatchDotChunkPtr;
		}

		bool bFileExists = true;
		if (DeletedRevision == nullptr)
		{
			if (bOnlyDeleted)
			{
				//skip
			}
			else if (Itr.Value.PakPriority == HighestPakPriority)
			{
				WriteCSVLine( FString::Printf( TEXT("%s,Fresh,%s,,%d,,%" INT64_FMT ",%s,%s,%s"), *AssetName, *Itr.Value.PakFilename, Itr.Value.PakPriority, Itr.Value.Size, *AssetPath, *PatchDotChunk, *OpenOrderText ) );
			}
			else
			{
				WriteCSVLine( FString::Printf( TEXT("%s,Inherited,%s,,%d,,%" INT64_FMT ",%s,%s,%s"), *AssetName, *Itr.Value.PakFilename, Itr.Value.PakPriority, Itr.Value.Size, *AssetPath, *PatchDotChunk, *OpenOrderText  ) );
			}
		}
		else if (DeletedRevision->PakPriority == Itr.Value.PakPriority)
		{
			WriteCSVLine( FString::Printf( TEXT("%s,Moved,%s,%s,%d,,%" INT64_FMT ",%s,%s,%s"), *AssetName, *Itr.Value.PakFilename, *DeletedRevision->PakFilename, Itr.Value.PakPriority, Itr.Value.Size, *AssetPath, *PatchDotChunk, *OpenOrderText ) );
		}
		else if (DeletedRevision->PakPriority > Itr.Value.PakPriority)
		{
			WriteCSVLine( FString::Printf( TEXT("%s,Deleted,%s,%s,%d,%d,,%s,%s,%s"), *AssetName, *DeletedRevision->PakFilename, *Itr.Value.PakFilename, DeletedRevision->PakPriority, Itr.Value.PakPriority, *AssetPath, *PatchDotChunk, *OpenOrderText ) );
			bFileExists = false;
		}
		else if (DeletedRevision->PakPriority < Itr.Value.PakPriority)
		{
			WriteCSVLine( FString::Printf( TEXT("%s,Restored,%s,%s,%d,%d,%" INT64_FMT ",%s,%s,%s"), *AssetName, *Itr.Value.PakFilename, *DeletedRevision->PakFilename, Itr.Value.PakPriority, DeletedRevision->PakPriority, Itr.Value.Size, *AssetPath, *PatchDotChunk, *OpenOrderText ) );
		}

		if( bFileExists && bSortByOrdering && bHasOpenOrder )
		{
			NumReads++;
			if( PreviousPatchDotChunk != PatchDotChunk )
			{ 
				PreviousPatchDotChunk = PatchDotChunk;
				NumSeeks++;
			}
		}
	}

	//check for deleted assets where there is no previous revision (missing pak files?)
	for (auto Itr : DeletedRevisions)
	{
		const FString& AssetPath = Itr.Key;
		const FFilePakRevision* Revision = FileRevisions.Find(AssetPath);
		if (Revision == nullptr)
		{
			//look up the open order for this file
			FString OpenOrderText = "";
			uint64 OpenOrder = FindOpenOrder(AssetPath);
			if( OpenOrder != UINT64_MAX )
			{
				OpenOrderText = FString::Printf( TEXT("%llu"), OpenOrder );
			}

			//lookup patch.chunk value
			FString PatchDotChunk = "";
			if( const FString* PatchDotChunkPtr = PakFilenameToPatchDotChunk.Find(Itr.Value.PakFilename) )
			{
				PatchDotChunk = *PatchDotChunkPtr;
			}

			const FString AssetName = FPaths::GetCleanFilename(AssetPath);
			WriteCSVLine( FString::Printf( TEXT("%s,Deleted,%s,Error,%d,,,%s,%s,%s"), *AssetName, *Itr.Value.PakFilename, Itr.Value.PakPriority, *AssetPath, *PatchDotChunk, *OpenOrderText ) );
		}
	}

	//clean up CSV writer
	if (CSVFileWriter)
	{
		CSVFileWriter->Close();
		delete CSVFileWriter;
		CSVFileWriter = NULL;
	}


	//write seek summary
	if( bSortByOrdering && bHasOpenOrder && NumReads > 0 )
	{
		UE_LOGF( LogPakFile, Display, "%d guaranteed seeks out of %d files read (%.2f%%) with the given open order", NumSeeks, NumReads, (float)(NumSeeks*100.0) / (float)NumReads );
	}

	return true;
}

bool ListFilesAtOffset( const TCHAR* InPakFileName, const TArray<int64>& InOffsets )
{
	if( InOffsets.Num() == 0 )
	{
		UE_LOGF(LogPakFile, Error, "No offsets specified" );
		return false;
	}

	TRefCountPtr<FPakFile> PakFilePtr = MakeRefCount<FPakFile>(&FPlatformFileManager::Get().GetPlatformFile(), InPakFileName, /*bIsSigned=*/ false);
	FPakFile& PakFile = *PakFilePtr;
	if (!PakFile.IsValid())
	{
		UE_LOGF(LogPakFile, Error, "Failed to open %ls", InPakFileName );
		return false;
	}

	UE_LOGF( LogPakFile, Display, "%-12ls%-12ls%-12ls%ls", TEXT("Offset"), TEXT("File Offset"), TEXT("File Size"), TEXT("File Name") );

	TArray<int64> OffsetsToCheck = InOffsets;
	FSharedPakReader PakReader = PakFile.GetSharedReader(NULL);
	for (FPakFile::FPakEntryIterator It(PakFile); It; ++It)
	{
		const FPakEntry& Entry = It.Info();

		//see if this file is on of the ones in the offset range we want
		int64 FoundOffset = INDEX_NONE;
		for( int64 Offset : OffsetsToCheck )
		{
			if( Offset >= Entry.Offset && Offset <= Entry.Offset+Entry.Size )
			{
				const FString* Filename = It.TryGetFilename();
				UE_LOG( LogPakFile, Display, TEXT("%-12" INT64_FMT "%-12" INT64_FMT "%-12" INT64_FMT "%s"), Offset, Entry.Offset, Entry.Size, Filename ? **Filename : TEXT("<FileNamesNotLoaded>") );
				FoundOffset = Offset;
				break;
			}
		}

		//remove it from the list if we found a match
		if( FoundOffset != INDEX_NONE )
		{
			OffsetsToCheck.Remove(FoundOffset);
		}
	}

	//list out any that we didn't find a match for
	for( int64 InvalidOffset : OffsetsToCheck )
	{
		UE_LOG(LogPakFile, Display, TEXT("%-12" INT64_FMT " - invalid offset"), InvalidOffset );
	}

	return true;
}

// used for diagnosing errors in FPakAsyncReadFileHandle::RawReadCallback
bool ShowCompressionBlockCRCs( const TCHAR* InPakFileName, TArray<int64>& InOffsets, const FKeyChain& InKeyChain )
{
	// open the pak file
	TRefCountPtr<FPakFile> PakFilePtr = MakeRefCount<FPakFile>(&FPlatformFileManager::Get().GetPlatformFile(), InPakFileName, /*bIsSigned=*/ false);
	FPakFile& PakFile = *PakFilePtr;
	if (!PakFile.IsValid())
	{
		UE_LOGF(LogPakFile, Error, "Failed to open %ls", InPakFileName );
		return false;
	}

	// read the pak file and iterate over all given offsets
	FSharedPakReader PakReader = PakFile.GetSharedReader(NULL);
	UE_LOGF(LogPakFile, Display, "" );
	for( int64 Offset : InOffsets )
	{
		//sanity check the offset
		if (Offset < 0 || Offset > PakFile.TotalSize() )
		{
			UE_LOG(LogPakFile, Error, TEXT("Offset: %" INT64_FMT " - out of range (max size is %" INT64_FMT ")"), Offset, PakFile.TotalSize() );
			continue;
		}

		//find the matching entry
		const FPakEntry* Entry = nullptr;
		for (FPakFile::FPakEntryIterator It(PakFile); It; ++It)
		{
			const FPakEntry& ThisEntry = It.Info();
			if( Offset >= ThisEntry.Offset && Offset <= ThisEntry.Offset+ThisEntry.Size )
			{
				Entry = &ThisEntry;
				FString EntryFilename = It.TryGetFilename() ? *It.TryGetFilename() : TEXT("<FileNamesNotLoaded>");
				FName EntryCompressionMethod = PakFile.GetInfo().GetCompressionMethod(Entry->CompressionMethodIndex);

				UE_LOG(LogPakFile, Display, TEXT("Offset: %" INT64_FMT "  -> EntrySize: %" INT64_FMT "  Encrypted: %-3s  Compression: %-8s  [%s]"), Offset, Entry->Size, Entry->IsEncrypted() ? TEXT("Yes") : TEXT("No"), *EntryCompressionMethod.ToString(), *EntryFilename );
				break;
			}
		}
		if (Entry == nullptr)
		{
			UE_LOGF(LogPakFile, Error, "Offset: %lld - no entry found.", Offset );
			continue;
		}

		// sanity check
		if (Entry->CompressionMethodIndex == 0)
		{
			UE_LOGF(LogPakFile, Error, "    Entry isn't compressed (not supported)" );
			continue;
		}
		if (Entry->IsDeleteRecord())
		{
			UE_LOGF(LogPakFile, Error, "    Entry is deleted" );
			continue;
		}

		//iterate over all blocks, decoding them and computing the checksum

		//... adapted from UncompressCopyFile...
		FName EntryCompressionMethod = PakFile.GetInfo().GetCompressionMethod(Entry->CompressionMethodIndex);
		int32 MaxCompressionBlockSize = FCompression::CompressMemoryBound(EntryCompressionMethod, Entry->CompressionBlockSize);
		for (const FPakCompressedBlock& Block : Entry->CompressionBlocks)
		{
			MaxCompressionBlockSize = FMath::Max<int32>(MaxCompressionBlockSize, IntCastChecked<int32>(Block.CompressedEnd - Block.CompressedStart));
		}

		int64 WorkingSize = Entry->CompressionBlockSize + MaxCompressionBlockSize;
		uint8* PersistentBuffer = (uint8*)FMemory::Malloc(WorkingSize);

		for (uint32 BlockIndex=0, BlockIndexNum=Entry->CompressionBlocks.Num(); BlockIndex < BlockIndexNum; ++BlockIndex)
		{
			int64 CompressedBlockSize = Entry->CompressionBlocks[BlockIndex].CompressedEnd - Entry->CompressionBlocks[BlockIndex].CompressedStart;
			PakReader->Seek(Entry->CompressionBlocks[BlockIndex].CompressedStart + (PakFile.GetInfo().HasRelativeCompressedChunkOffsets() ? Entry->Offset : 0));
			int64 SizeToRead = Entry->IsEncrypted() ? Align(CompressedBlockSize, FAES::AESBlockSize) : CompressedBlockSize;
			PakReader->Serialize(PersistentBuffer, SizeToRead);

			if (Entry->IsEncrypted())
			{
				const FNamedAESKey* Key = InKeyChain.GetPrincipalEncryptionKey();
				check(Key);
				FAES::DecryptData(PersistentBuffer, SizeToRead, Key->Key);
			}

			// adapted from FPakAsyncReadFileHandle::RawReadCallback
			int32 ProcessedSize = Entry->CompressionBlockSize;
			if (BlockIndex == BlockIndexNum - 1)
			{
				ProcessedSize = Entry->UncompressedSize % Entry->CompressionBlockSize;
				if (!ProcessedSize)
				{
					ProcessedSize = Entry->CompressionBlockSize; // last block was a full block
				}
			}

			// compute checksum and log out the block information
			uint32 TruncatedCompressedBlockSize = IntCastChecked<uint32>(CompressedBlockSize);
			const uint32 BlockCrc32 = FCrc::MemCrc32(PersistentBuffer, TruncatedCompressedBlockSize);
			const FString HexBytes = BytesToHex(PersistentBuffer,FMath::Min(TruncatedCompressedBlockSize,32U));
			UE_LOG(LogPakFile, Display, TEXT("    Block:%-6d  ProcessedSize: %-6d  DecompressionRawSize: %-6" INT64_FMT "  Crc32: %-12u [%s...]"), BlockIndex, ProcessedSize, CompressedBlockSize, BlockCrc32, *HexBytes );
		}

		FMemory::Free(PersistentBuffer);
		UE_LOGF(LogPakFile, Display, "" );

	}

	UE_LOGF(LogPakFile, Display, "done" );
	return true;
}

bool GeneratePIXMappingFile(const TArray<FString> InPakFileList, const FString& OutputPath)
{
	if (!InPakFileList.Num())
	{
		UE_LOGF(LogPakFile, Error, "Pak file list can not be empty.");
		return false;
	}

	if (!FPaths::DirectoryExists(OutputPath))
	{
		UE_LOGF(LogPakFile, Error, "Output path doesn't exist.  Create %ls.", *OutputPath);
		FPlatformFileManager::Get().GetPlatformFile().CreateDirectoryTree(*OutputPath);
	}

	for (const FString& PakFileName : InPakFileList)
	{
		// open CSV file, if requested
		FArchive* CSVFileWriter = nullptr;
		FString OutputMappingFilename = OutputPath / FPaths::GetBaseFilename(PakFileName) + TEXT(".csv");
		if (!OutputMappingFilename.IsEmpty())
		{
			CSVFileWriter = IFileManager::Get().CreateFileWriter(*OutputMappingFilename);
			if (CSVFileWriter == nullptr)
			{
				UE_LOGF(LogPakFile, Error, "Unable to open csv file \"%ls\".", *OutputMappingFilename);
				return false;
			}
		}

		TRefCountPtr<FPakFile> PakFilePtr = MakeRefCount<FPakFile>(&FPlatformFileManager::Get().GetPlatformFile(), *PakFileName, /*bIsSigned=*/ false);
		FPakFile& PakFile = *PakFilePtr;
		if (!PakFile.IsValid())
		{
			UE_LOGF(LogPakFile, Error, "Failed to open %ls", *PakFileName);
			return false;
		}

		CSVFileWriter->Logf(TEXT("%s"), *PakFileName);

		const FString PakFileMountPoint = PakFile.GetMountPoint();
		FSharedPakReader PakReader = PakFile.GetSharedReader(NULL);
		if (!PakFile.HasFilenames())
		{
			UE_LOGF(LogPakFile, Error, "PakFiles were loaded without filenames, cannot generate PIX mapping file.");
			return false;
		}

		for (FPakFile::FPakEntryIterator It(PakFile); It; ++It)
		{
			const FPakEntry& Entry = It.Info();

			CSVFileWriter->Logf(TEXT("0x%010" INT64_FMT ",0x%08" INT64_FMT ",%s"), Entry.Offset, Entry.Size, *(PakFileMountPoint / *It.TryGetFilename()));
		}

		CSVFileWriter->Close();
		delete CSVFileWriter;
		CSVFileWriter = nullptr;
	}

	return true;
}

bool ExtractFilesFromPak(const TCHAR* InPakFilename, TMap<FString, FFileInfo>& InFileHashes, 
	const TCHAR* InDestPath, bool bUseMountPoint, const FKeyChain& InKeyChain, const FString* InFilter, 
	TArray<FPakInputPair>* OutEntries, TArray<FPakInputPair>* OutDeletedEntries, FPakOrderMap* OutOrderMap, 
	TArray<FGuid>* OutUsedEncryptionKeys, bool* OutAnyPakSigned)
{
	// Gather all patch versions of the requested pak file and run through each separately
	TArray<FString> PakFileList;
	FString PakFileDirectory = FPaths::GetPath(InPakFilename);
	// If file doesn't exist try using it as a search string, it may contain wild cards
	if (IFileManager::Get().FileExists(InPakFilename))
	{
		PakFileList.Add(*FPaths::GetCleanFilename(InPakFilename));
	}
	else
	{
		IFileManager::Get().FindFiles(PakFileList, *PakFileDirectory, *FPaths::GetCleanFilename(InPakFilename));
	}

	if (PakFileList.Num() == 0)
	{
		// No files found
		return false;
	}

	if (OutOrderMap)
	{
		if (PakFileList.Num() > 1)
		{
			UE_LOGF(LogPakFile, Error,
				"ExtractFilesFromPak: Collecting the order of the files in a pak is not implemented when multiple packs are specified by wildcard. "
				"ExtractFilesFromPak was called on '%ls', which matched %d pakfiles.",
				InPakFilename, PakFileList.Num());
			return false;
		}
		OutOrderMap->Empty();
	}


	bool bIncludeDeleted = (OutDeletedEntries != nullptr);

	const FString DestPath = FPaths::ConvertRelativePathToFull(InDestPath);
	for (int32 PakFileIndex = 0; PakFileIndex < PakFileList.Num(); PakFileIndex++)
	{
		FString PakFilename = PakFileDirectory + "\\" + PakFileList[PakFileIndex];
		int32 PakPriority = GetPakPriorityFromFilename(PakFilename);

		TRefCountPtr<FPakFile> PakFilePtr = MakeRefCount<FPakFile>(&FPlatformFileManager::Get().GetPlatformFile(), *PakFilename, /*bIsSigned=*/ false);
		FPakFile& PakFile = *PakFilePtr;
		if (PakFile.IsValid())
		{
			if (OutUsedEncryptionKeys)
			{
				OutUsedEncryptionKeys->Add(PakFile.GetInfo().EncryptionKeyGuid);
			}

			if (OutAnyPakSigned && *OutAnyPakSigned == false)
			{
				FString SignatureFile(FPaths::ChangeExtension(PakFilename, TEXT(".sig")));
				if (FPaths::FileExists(SignatureFile))
				{
					*OutAnyPakSigned = true;
				}
			}

			FString PakMountPoint = bUseMountPoint ? PakFile.GetMountPoint().Replace(TEXT("../../../"), TEXT("")) : TEXT("");
			if (!PakFile.HasFilenames())
			{
				UE_LOGF(LogPakFile, Error, "PakFiles were loaded without filenames, cannot extract.");
				return false;
			}

			FSharedPakReader PakReader = PakFile.GetSharedReader(NULL);
			const int64 BufferSize = 8 * 1024 * 1024; // 8MB buffer for extracting
			void* Buffer = FMemory::Malloc(BufferSize);
			int64 CompressionBufferSize = 0;
			uint8* PersistentCompressionBuffer = NULL;
			int32 ErrorCount = 0;
			int32 FileCount = 0;
			int32 ExtractedCount = 0;

			for (FPakFile::FPakEntryIterator It(PakFile,bIncludeDeleted); It; ++It, ++FileCount)
			{
				// Extract only the most recent version of a file when present in multiple paks
				const FString& EntryFileName = *It.TryGetFilename();
				FFileInfo* HashFileInfo = InFileHashes.Find(EntryFileName);
				if (HashFileInfo == nullptr || HashFileInfo->PatchIndex == PakPriority)
				{
					FString DestFilename = FPaths::ConvertRelativePathToFull(DestPath / PakMountPoint / EntryFileName);

					const FPakEntry& Entry = It.Info();
					if (Entry.IsDeleteRecord())
					{
						checkSlow(OutDeletedEntries);
						UE_LOGF(LogPakFile, Display, "Found delete record for \"%ls\".", *EntryFileName);

						FPakInputPair DeleteRecord;
						DeleteRecord.bIsDeleteRecord = true;
						DeleteRecord.Source = DestFilename;
						DeleteRecord.Dest = PakFile.GetMountPoint() / EntryFileName;
						OutDeletedEntries->Add(DeleteRecord);
						continue;
					}

					if (InFilter && (!EntryFileName.MatchesWildcard(*InFilter)))
					{
						continue;
					}

					if (!FPaths::IsUnderDirectory(DestFilename, DestPath))
					{
						FString EntryPath = FPaths::GetPath(EntryFileName);
						uint64 EntryHash = FXxHash64::HashBuffer(*EntryPath, EntryPath.Len() * sizeof(TCHAR)).Hash;
						FString HashDir = FString::Printf(TEXT("%016llx"), EntryHash);
						FString SafeDestFilename = FPaths::ConvertRelativePathToFull(DestPath / HashDir / FPaths::GetCleanFilename(EntryFileName));
						UE_LOGF(LogPakFile, Display, "Fixup extract path for \"%ls\": from \"%ls\" to \"%ls\"",
							*EntryFileName, *DestFilename, *SafeDestFilename);
						DestFilename = SafeDestFilename;
					}

					PakReader->Seek(Entry.Offset);
					uint32 SerializedCrcTest = 0;
					FPakEntry EntryInfo;
					EntryInfo.Serialize(PakReader.GetArchive(), PakFile.GetInfo().Version);
					if (EntryInfo.IndexDataEquals(Entry))
					{
						TUniquePtr<FArchive> FileHandle(IFileManager::Get().CreateFileWriter(*DestFilename));
						if (FileHandle)
						{
							if (Entry.CompressionMethodIndex == 0)
							{
								BufferedCopyFile(*FileHandle, PakReader.GetArchive(), PakFile, Entry, Buffer, BufferSize, InKeyChain);
							}
							else
							{
								UncompressCopyFile(*FileHandle, PakReader.GetArchive(), Entry, PersistentCompressionBuffer, CompressionBufferSize, InKeyChain, PakFile);
							}
							UE_LOGF(LogPakFile, Display, "Extracted \"%ls\" to \"%ls\" Offset %lld.", *EntryFileName, *DestFilename, Entry.Offset);
							ExtractedCount++;

							if (OutOrderMap != nullptr)
							{
								OutOrderMap->AddOffset(PakFile.GetMountPoint() / EntryFileName, It.Info().Offset);
							}

							if (OutEntries != nullptr)
							{
								FPakInputPair Input;

								Input.Source = DestFilename;
								FPaths::NormalizeFilename(Input.Source);

								Input.Dest = PakFile.GetMountPoint() / EntryFileName;

								Input.bNeedsCompression = Entry.CompressionMethodIndex != 0;
								Input.bNeedEncryption = Entry.IsEncrypted();
	
								OutEntries->Add(Input);
							}
						}
						else
						{
							UE_LOGF(LogPakFile, Error, "Unable to create file \"%ls\".", *DestFilename);
							ErrorCount++;
						}
					}
					else
					{
						UE_LOGF(LogPakFile, Error, "PakEntry mismatch for \"%ls\".", *EntryFileName);
						ErrorCount++;
					}
				}
			}
			FMemory::Free(Buffer);
			FMemory::Free(PersistentCompressionBuffer);

			UE_LOGF(LogPakFile, Log, "Finished extracting %d (including %d errors).", ExtractedCount, ErrorCount);
		}
		else
		{
			TMap<FString, uint64> IoStoreContainerOrderMap;
			bool bIoStoreContainerIsSigned;
			if (!ExtractFilesFromIoStoreContainer(
				*PakFilename,
				InDestPath,
				InKeyChain,
				InFilter,
				OutOrderMap ? &IoStoreContainerOrderMap : nullptr,
				OutUsedEncryptionKeys,
				&bIoStoreContainerIsSigned))
			{
				UE_LOGF(LogPakFile, Error, "Unable to open pak file \"%ls\".", *PakFilename);
				return false;
			}

			if (OutEntries)
			{
				UE_LOGF(LogPakFile, Warning, "Generating response files from IoStore containers is not supported.");
			}

			if (OutOrderMap)
			{
				for (const auto& KV : IoStoreContainerOrderMap)
				{
					OutOrderMap->AddOffset(KV.Key, KV.Value);
				}
			}

			if (OutAnyPakSigned)
			{
				*OutAnyPakSigned |= bIoStoreContainerIsSigned;
			}
		}
	}

	if (OutOrderMap)
	{
		OutOrderMap->ConvertOffsetsToOrder();
	}

	return true;
}

void CreateDiffRelativePathMap(TArray<FString>& FileNames, const FString& RootPath, TMap<FName, FString>& OutMap)
{
	for (int32 i = 0; i < FileNames.Num(); ++i)
	{
		const FString& FullPath = FileNames[i];
		FString RelativePath = FullPath.Mid(RootPath.Len());
		OutMap.Add(FName(*RelativePath), FullPath);
	}
}

bool DumpPakInfo(const FString& InPakFilename, const FKeyChain& InKeyChain)
{
	TRefCountPtr<FPakFile> PakFilePtr = MakeRefCount<FPakFile>(&FPlatformFileManager::Get().GetPlatformFile(), *InPakFilename, /*bIsSigned=*/ false);
	FPakFile& PakFile = *PakFilePtr;

	if (!PakFile.IsValid())
	{
		if (DumpIoStoreContainerInfo(*InPakFilename, InKeyChain))
		{
			return true;
		}
		else
		{
			return false;
		}
	}


	const FPakInfo& Info = PakFile.GetInfo();

	UE_LOGF(LogPakFile, Display, "Pak File: %ls", *InPakFilename);
	UE_LOGF(LogPakFile, Display, "    Version: %d", Info.Version);
	UE_LOG(LogPakFile, Display, TEXT("    IndexOffset: %" INT64_FMT), Info.IndexOffset);
	UE_LOG(LogPakFile, Display, TEXT("    IndexSize: %" INT64_FMT), Info.IndexSize);
	UE_LOGF(LogPakFile, Display, "    IndexHash: %ls", *Info.IndexHash.ToString());
	UE_LOGF(LogPakFile, Display, "    bEncryptedIndex: %d", Info.bEncryptedIndex);
	UE_LOGF(LogPakFile, Display, "    EncryptionKeyGuid: %ls", *Info.EncryptionKeyGuid.ToString());
	UE_LOGF(LogPakFile, Display, "    CompressionMethods:");
	for (FName Method : Info.CompressionMethods)
	{
		UE_LOGF(LogPakFile, Display, "        %ls", *Method.ToString());
	}

	return true;
}

bool DiffFilesInPaks(const FString& InPakFilename1, const FString& InPakFilename2, const bool bLogUniques1, const bool bLogUniques2, const FKeyChain& InKeyChain1, const FKeyChain& InKeyChain2)
{
	int32 NumUniquePAK1 = 0;
	int32 NumUniquePAK2 = 0;
	int32 NumDifferentContents = 0;
	int32 NumEqualContents = 0;

	TGuardValue DisableLogTimes(GPrintLogTimes, ELogTimes::None);

	TRefCountPtr<FPakFile> PakFilePtr1 = MakeRefCount<FPakFile>(&FPlatformFileManager::Get().GetPlatformFile(), *InPakFilename1, /*bIsSigned=*/ false);
	FPakFile& PakFile1 = *PakFilePtr1;
	TRefCountPtr<FPakFile> PakFilePtr2 = MakeRefCount<FPakFile>(&FPlatformFileManager::Get().GetPlatformFile(), *InPakFilename2, /*bIsSigned=*/ false);
	FPakFile& PakFile2 = *PakFilePtr2;
	if (PakFile1.IsValid() && PakFile2.IsValid())
	{		
		UE_LOGF(LogPakFile, Log, "FileEventType, FileName, Size1, Size2");

		if (!PakFile1.HasFilenames() || !PakFile2.HasFilenames())
		{
			UE_LOGF(LogPakFile, Error, "Pakfiles were loaded without Filenames, cannot diff.");
			return false;
		}

		FSharedPakReader PakReader1 = PakFile1.GetSharedReader(NULL);
		FSharedPakReader PakReader2 = PakFile2.GetSharedReader(NULL);

		const int64 BufferSize = 8 * 1024 * 1024; // 8MB buffer for extracting
		void* Buffer = FMemory::Malloc(BufferSize);
		int64 CompressionBufferSize = 0;
		uint8* PersistentCompressionBuffer = NULL;
		int32 ErrorCount = 0;
		int32 FileCount = 0;

		//loop over pak1 entries.  compare against entry in pak2.
		for (FPakFile::FPakEntryIterator It(PakFile1); It; ++It, ++FileCount)
		{
			const FString& PAK1FileName = *It.TryGetFilename();

			//double check entry info and move pakreader into place
			const FPakEntry& Entry1 = It.Info();
			PakReader1->Seek(Entry1.Offset);

			FPakEntry EntryInfo1;
			EntryInfo1.Serialize(PakReader1.GetArchive(), PakFile1.GetInfo().Version);

			if (!EntryInfo1.IndexDataEquals(Entry1))
			{
				UE_LOGF(LogPakFile, Log, "PakEntry1Invalid, %ls, 0, 0", *PAK1FileName);
				continue;
			}
			
			//see if entry exists in other pak							
			FPakEntry Entry2;
			FPakFile::EFindResult FoundEntry2 = PakFile2.Find(PakFile1.GetMountPoint() / PAK1FileName, &Entry2);
			if (FoundEntry2 != FPakFile::EFindResult::Found)
			{
				++NumUniquePAK1;
				if (bLogUniques1)
				{
					UE_LOG(LogPakFile, Log, TEXT("UniqueToFirstPak, %s, %" INT64_FMT ", 0"), *PAK1FileName, EntryInfo1.UncompressedSize);
				}
				continue;
			}

			//double check entry info and move pakreader into place
			PakReader2->Seek(Entry2.Offset);
			FPakEntry EntryInfo2;
			EntryInfo2.Serialize(PakReader2.GetArchive(), PakFile2.GetInfo().Version);
			if (!EntryInfo2.IndexDataEquals(Entry2))
			{
				UE_LOGF(LogPakFile, Log, "PakEntry2Invalid, %ls, 0, 0", *PAK1FileName);
				continue;;
			}

			//check sizes first as quick compare.
			if (EntryInfo1.UncompressedSize != EntryInfo2.UncompressedSize)
			{
				UE_LOG(LogPakFile, Log, TEXT("FilesizeDifferent, %s, %" INT64_FMT ", %" INT64_FMT), *PAK1FileName, EntryInfo1.UncompressedSize, EntryInfo2.UncompressedSize);
				continue;
			}
			
			//serialize and memcompare the two entries
			{
				FLargeMemoryWriter PAKWriter1(EntryInfo1.UncompressedSize);
				FLargeMemoryWriter PAKWriter2(EntryInfo2.UncompressedSize);

				if (EntryInfo1.CompressionMethodIndex == 0)
				{
					BufferedCopyFile(PAKWriter1, PakReader1.GetArchive(), PakFile1, Entry1, Buffer, BufferSize, InKeyChain1);
				}
				else
				{
					UncompressCopyFile(PAKWriter1, PakReader1.GetArchive(), Entry1, PersistentCompressionBuffer, CompressionBufferSize, InKeyChain1, PakFile1);
				}

				if (EntryInfo2.CompressionMethodIndex == 0)
				{
					BufferedCopyFile(PAKWriter2, PakReader2.GetArchive(), PakFile2, Entry2, Buffer, BufferSize, InKeyChain2);
				}
				else
				{
					UncompressCopyFile(PAKWriter2, PakReader2.GetArchive(), Entry2, PersistentCompressionBuffer, CompressionBufferSize, InKeyChain2, PakFile2);
				}

				if (FMemory::Memcmp(PAKWriter1.GetData(), PAKWriter2.GetData(), EntryInfo1.UncompressedSize) != 0)
				{
					++NumDifferentContents;
					UE_LOG(LogPakFile, Log, TEXT("ContentsDifferent, %s, %" INT64_FMT ", %" INT64_FMT), *PAK1FileName, EntryInfo1.UncompressedSize, EntryInfo2.UncompressedSize);
				}
				else
				{
					++NumEqualContents;
				}
			}			
		}

		//check for files unique to the second pak.
		for (FPakFile::FPakEntryIterator It(PakFile2); It; ++It, ++FileCount)
		{
			const FPakEntry& Entry2 = It.Info();
			PakReader2->Seek(Entry2.Offset);

			FPakEntry EntryInfo2;
			EntryInfo2.Serialize(PakReader2.GetArchive(), PakFile2.GetInfo().Version);

			if (EntryInfo2.IndexDataEquals(Entry2))
			{
				const FString& PAK2FileName = *It.TryGetFilename();
				FPakEntry Entry1;
				FPakFile::EFindResult FoundEntry1 = PakFile1.Find(PakFile2.GetMountPoint() / PAK2FileName, &Entry1);
				if (FoundEntry1 != FPakFile::EFindResult::Found)
				{
					++NumUniquePAK2;
					if (bLogUniques2)
					{
						UE_LOG(LogPakFile, Log, TEXT("UniqueToSecondPak, %s, 0, %" INT64_FMT), *PAK2FileName, Entry2.UncompressedSize);
					}
					continue;
				}
			}
		}

		FMemory::Free(Buffer);
		Buffer = nullptr;

		UE_LOGF(LogPakFile, Display, "Unique to first pak: %i, Unique to second pak: %i, Num Different: %i, NumEqual: %i", NumUniquePAK1, NumUniquePAK2, NumDifferentContents, NumEqualContents);
		return true;
	}
	else if (LegacyDiffIoStoreContainers(*InPakFilename1, *InPakFilename2, bLogUniques1, bLogUniques2, InKeyChain1, &InKeyChain2))
	{
		return true;
	}

	return false;
}

void GenerateHashForFile(uint8* ByteBuffer, uint64 TotalSize, FFileInfo& FileHash)
{
	FMD5 FileHasher;
	FileHasher.Update(ByteBuffer, TotalSize);
	FileHasher.Final(FileHash.Hash);
	FileHash.FileSize = TotalSize;
}

bool GenerateHashForFile( FString Filename, FFileInfo& FileHash)
{
	FArchive* File = IFileManager::Get().CreateFileReader(*Filename);

	if ( File == NULL )
		return false;

	uint64 TotalSize = File->TotalSize();

	uint8* ByteBuffer = new uint8[TotalSize];

	File->Serialize(ByteBuffer, TotalSize);

	delete File;
	File = NULL;

	GenerateHashForFile(ByteBuffer, TotalSize, FileHash);
	
	delete[] ByteBuffer;
	return true;
}

bool GenerateHashesFromPak(const TCHAR* InPakFilename, const TCHAR* InDestPakFilename, TMap<FString, FFileInfo>& FileHashes, bool bUseMountPoint, const FKeyChain& InKeyChain, int32& OutLowestSourcePakVersion, TArray<FGuid>* OutUsedEncryptionKeys = nullptr)
{
	OutLowestSourcePakVersion = FPakInfo::PakFile_Version_Invalid;

	// Gather all patch pak files and run through them one at a time
	TArray<FString> PakFileList;
	IFileManager::Get().FindFiles(PakFileList, InPakFilename, true, false);

	FString PakFileDirectory = FPaths::GetPath(InPakFilename);

	for (int32 PakFileIndex = 0; PakFileIndex < PakFileList.Num(); PakFileIndex++)
	{
		FString PakFilename = PakFileDirectory + "\\" + PakFileList[PakFileIndex];
		// Skip the destination pak file so we can regenerate an existing patch level
		if (PakFilename.Equals(InDestPakFilename))
		{
			continue;
		}
		int32 PakPriority = GetPakPriorityFromFilename(PakFilename);
		int32 PakChunkIndex = GetPakChunkIndexFromFilename(PakFilename);

		TRefCountPtr<FPakFile> PakFilePtr = MakeRefCount<FPakFile>(&FPlatformFileManager::Get().GetPlatformFile(), *PakFilename, /*bIsSigned=*/ false);
		FPakFile& PakFile = *PakFilePtr;
		if (PakFile.IsValid())
		{
			if (OutUsedEncryptionKeys != nullptr)
			{
				OutUsedEncryptionKeys->Add( PakFile.GetInfo().EncryptionKeyGuid);
			}

			FString PakMountPoint = bUseMountPoint ? PakFile.GetMountPoint().Replace(TEXT("../../../"), TEXT("")) : TEXT("");
			if (!PakFile.HasFilenames())
			{
				UE_LOGF(LogPakFile, Error, "Pakfiles were loaded without Filenames, cannot GenerateHashesFromPak.");
				return false;
			}

			FSharedPakReader PakReader = PakFile.GetSharedReader(NULL);
			const int64 BufferSize = 8 * 1024 * 1024; // 8MB buffer for extracting
			void* Buffer = FMemory::Malloc(BufferSize);
			int64 CompressionBufferSize = 0;
			uint8* PersistentCompressionBuffer = NULL;
			int32 ErrorCount = 0;
			int32 FileCount = 0;

			//remember the lowest pak version for any patch paks
			if( PakPriority != -1 )
			{
				OutLowestSourcePakVersion = FMath::Min( OutLowestSourcePakVersion, PakFile.GetInfo().Version );
			}

			const bool bIncludeDeleted = true;
			for (FPakFile::FPakEntryIterator It(PakFile,bIncludeDeleted); It; ++It, ++FileCount)
			{
				const FPakEntry& Entry = It.Info();
				FFileInfo FileHash = {};
				bool bEntryValid = false;

				FString FullFilename = PakMountPoint;
				if (!FullFilename.IsEmpty() && !FullFilename.EndsWith("/"))
				{
					FullFilename += "/";
				}
				FullFilename += *It.TryGetFilename();

				if (Entry.IsDeleteRecord())
				{
					FileHash.PatchIndex = PakPriority;
					FileHash.bIsDeleteRecord = true;
					FileHash.bForceInclude = false;
					bEntryValid = true;
				}
				else
				{
					PakReader->Seek(Entry.Offset);
					uint32 SerializedCrcTest = 0;
					FPakEntry EntryInfo;
					EntryInfo.Serialize(PakReader.GetArchive(), PakFile.GetInfo().Version);
					if (EntryInfo.IndexDataEquals(Entry))
					{
						// TUniquePtr<FArchive> FileHandle(IFileManager::Get().CreateFileWriter(*DestFilename));
						FLargeMemoryWriter MemoryFile;
						FArchive* FileHandle = &MemoryFile;
						// if (FileHandle.IsValid())
						{
							if (Entry.CompressionMethodIndex == 0)
							{
								BufferedCopyFile(*FileHandle, PakReader.GetArchive(), PakFile, Entry, Buffer, BufferSize, InKeyChain);
							}
							else
							{
								UncompressCopyFile(*FileHandle, PakReader.GetArchive(), Entry, PersistentCompressionBuffer, CompressionBufferSize, InKeyChain, PakFile);
							}

							UE_LOGF(LogPakFile, Display, "Generated hash for \"%ls\"", *FullFilename);
							GenerateHashForFile(MemoryFile.GetData(), MemoryFile.TotalSize(), FileHash);
							FileHash.PatchIndex = PakPriority;
							FileHash.bIsDeleteRecord = false;
							FileHash.bForceInclude = false;
							bEntryValid = true;
						}
						/*else
						{
						UE_LOGF(LogPakFile, Error, "Unable to create file \"%ls\".", *DestFilename);
						ErrorCount++;
						}*/

					}
					else
					{
						UE_LOGF(LogPakFile, Error, "Serialized hash mismatch for \"%ls\".", **It.TryGetFilename());
						ErrorCount++;
					}
				}

				if (bEntryValid)
				{
					// Keep only the hash of the most recent version of a file (across multiple pak patch files)
					if (!FileHashes.Contains(FullFilename))
					{
						FileHashes.Add(FullFilename, FileHash);
					}
					else if (FileHashes[FullFilename].PatchIndex < FileHash.PatchIndex)
					{
						FileHashes[FullFilename] = FileHash;
					}
				}
			}
			FMemory::Free(Buffer);
			FMemory::Free(PersistentCompressionBuffer);

			UE_LOGF(LogPakFile, Log, "Finished extracting %d files (including %d errors).", FileCount, ErrorCount);
		}
		else
		{
			UE_LOGF(LogPakFile, Error, "Unable to open pak file \"%ls\".", *PakFilename);
			return false;
		}
	}

	return true;
}

bool FileIsIdentical(FString SourceFile, FString DestFilename, const FFileInfo* Hash, int64* DestSizeOut = nullptr)
{
	int64 SourceTotalSize = Hash ? Hash->FileSize : IFileManager::Get().FileSize(*SourceFile);
	int64 DestTotalSize = IFileManager::Get().FileSize(*DestFilename);

	if (DestSizeOut != nullptr)
	{
		*DestSizeOut = DestTotalSize;
	}

	if (SourceTotalSize != DestTotalSize)
	{
		// file size doesn't match 
		UE_LOG(LogPakFile, Display, TEXT("Source file size for %s %" INT64_FMT " bytes doesn't match %s %" INT64_FMT " bytes, did find %d"), *SourceFile, SourceTotalSize, *DestFilename, DestTotalSize, Hash ? 1 : 0);
		return false;
	}

	FFileInfo SourceFileHash;
	if (!Hash)
	{
		if (GenerateHashForFile(SourceFile, SourceFileHash) == false)
		{
			// file size doesn't match 
			UE_LOGF(LogPakFile, Display, "Source file size %ls doesn't exist will be included in build", *SourceFile);
			return false;;
		}
		else
		{
			UE_LOGF(LogPakFile, Warning, "Generated hash for file %ls but it should have been in the FileHashes array", *SourceFile);
		}
	}
	else
	{
		SourceFileHash = *Hash;
	}

	FFileInfo DestFileHash;
	if (GenerateHashForFile(DestFilename, DestFileHash) == false)
	{
		// destination file was removed don't really care about it
		UE_LOGF(LogPakFile, Display, "File was removed from destination cooked content %ls not included in patch", *DestFilename);
		return false;
	}

	int32 Diff = FMemory::Memcmp(&SourceFileHash.Hash, &DestFileHash.Hash, sizeof(DestFileHash.Hash));
	if (Diff != 0)
	{
		UE_LOGF(LogPakFile, Display, "Source file hash for %ls doesn't match dest file hash %ls and will be included in patch", *SourceFile, *DestFilename);
		return false;
	}

	return true;
}

float GetFragmentationPercentage(const TArray<FPakInputPair>& FilesToPak, const TBitArray<>& IncludeBitMask, int64 MaxAdjacentOrderDiff, bool bConsiderSecondaryFiles)
{
	uint64 PrevOrder = MAX_uint64;
	bool bPrevBit = false;
	int32 DiffCount = 0;
	int32 ConsideredCount = 0;
	for (int32 i = 0; i < IncludeBitMask.Num(); i++)
	{
		if (!bConsiderSecondaryFiles && !FilesToPak[i].bIsInPrimaryOrder)
		{
			PrevOrder = MAX_uint64;
			continue;
		}
		uint64 CurrentOrder = FilesToPak[i].SuggestedOrder;
		bool bCurrentBit = IncludeBitMask[i];
		uint64 OrderDiff = CurrentOrder - PrevOrder;
		if (OrderDiff > (uint64)(MaxAdjacentOrderDiff) || bCurrentBit != bPrevBit)
		{
			DiffCount++;
		}
		ConsideredCount++;
		PrevOrder = CurrentOrder;
		bPrevBit = bCurrentBit;
	}
	// First always shows as different, so discount it
	DiffCount--;
	return ConsideredCount ? (100.0f * float(DiffCount) / float(ConsideredCount)) : 0;
}

int64 ComputePatchSize(const TArray<FPakInputPair>& FilesToPak, const TArray<int64>& FileSizes, TBitArray<>& IncludeFilesMask, int32& OutFileCount)
{
	int64 TotalPatchSize = 0;
	OutFileCount = 0;
	for (int i = 0; i < FilesToPak.Num(); i++)
	{
		if (IncludeFilesMask[i])
		{
			OutFileCount++;
			TotalPatchSize += FileSizes[i];
		}
	}
	return TotalPatchSize;
}

int32 AddOrphanedFiles(const TArray<FPakInputPair>& FilesToPak, const TArray<int64>& FileSizes, const TArray<int32>& UAssetToUexpMapping, TBitArray<>& IncludeFilesMask, int64& OutSizeIncrease)
{
	int32 UExpCount = 0;
	int32 UAssetCount = 0;
	int32 FilesAddedCount = 0;
	OutSizeIncrease = 0;
	// Add corresponding UExp/UBulk files to the patch if either is included but not the other
	for (int i = 0; i < FilesToPak.Num(); i++)
	{
		int32 CounterpartFileIndex = UAssetToUexpMapping[i];
		if (CounterpartFileIndex != -1)
		{
			const FPakInputPair& File = FilesToPak[i];
			const FPakInputPair& CounterpartFile = FilesToPak[CounterpartFileIndex];
			if (IncludeFilesMask[i] != IncludeFilesMask[CounterpartFileIndex])
			{
				if (!IncludeFilesMask[i])
				{
					//UE_LOGF(LogPakFile, Display, "Added %ls because %ls is already included", *FilesToPak[i].Source, *FilesToPak[CounterpartFileIndex].Source);
					IncludeFilesMask[i] = true;
					OutSizeIncrease += FileSizes[i];
				}
				else
				{
					//UE_LOGF(LogPakFile, Display, "Added %ls because %ls is already included", *FilesToPak[CounterpartFileIndex].Source, *FilesToPak[i].Source);
					IncludeFilesMask[CounterpartFileIndex] = true;
					OutSizeIncrease += FileSizes[CounterpartFileIndex];
				}
				FilesAddedCount++;
			}
		}
	}
	return FilesAddedCount;
}


bool DoGapFillingIteration(const TArray<FPakInputPair>& FilesToPak, const TArray<int64>& FileSizes, const TArray<int32>& UAssetToUexpMapping, TBitArray<>& InOutIncludeFilesMask, int64 MaxGapSizeBytes, int64 MaxAdjacentOrderDiff, bool bForce, int64 MaxPatchSize = 0, bool bFillPrimaryOrderFiles = true, bool bFillSecondaryOrderFiles = true)
{
	TBitArray<> IncludeFilesMask = InOutIncludeFilesMask;

	float FragmentationPercentageOriginal = GetFragmentationPercentage(FilesToPak, IncludeFilesMask, MaxAdjacentOrderDiff, true);

	int64 SizeIncrease = 0;
	int64 CurrentOffset = 0;
	int64 CurrentPatchOffset = 0;
	int64 CurrentGapSize = 0;
	bool bPrevKeepFile = false;
	uint64 PrevOrder = MAX_uint64;
	int32 OriginalKeepCount = 0;
	int32 LastKeepIndex = -1;
	bool bCurrentGapIsUnbroken = true;
	int32 PatchFilesAddedCount = 0;
	for (int i = 0; i < FilesToPak.Num(); i++)
	{
		bool bKeepFile = IncludeFilesMask[i];
		const FPakInputPair& File = FilesToPak[i];
		uint64 Order = File.SuggestedOrder;

		// Skip unordered files or files outside the range we care about
		if (Order == MAX_uint64 || (File.bIsInPrimaryOrder && !bFillPrimaryOrderFiles) || (!File.bIsInPrimaryOrder && !bFillSecondaryOrderFiles))
		{
			continue;
		}
		CurrentOffset += FileSizes[i];
		if (bKeepFile)
		{
			OriginalKeepCount++;
			CurrentPatchOffset = CurrentOffset;
		}
		else
		{
			if (OriginalKeepCount > 0)
			{
				CurrentGapSize = CurrentOffset - CurrentPatchOffset;
			}
		}

		// Detect gaps in the file order. No point in removing those gaps because it won't affect seeks
		uint64 OrderDiff = Order - PrevOrder;
		if (bCurrentGapIsUnbroken && OrderDiff > uint64(MaxAdjacentOrderDiff))
		{
			bCurrentGapIsUnbroken = false;
		}

		// If we're keeping this file but not the last one, check if the gap size is small enough to bring over unchanged assets
		if (bKeepFile && !bPrevKeepFile && CurrentGapSize > 0)
		{
			if (CurrentGapSize <= MaxGapSizeBytes)
			{
				if (bCurrentGapIsUnbroken)
				{
					// Mark the files in the gap to keep, even though they're unchanged
					for (int j = LastKeepIndex + 1; j < i; j++)
					{
						IncludeFilesMask[j] = true;
						SizeIncrease += FileSizes[j];
						PatchFilesAddedCount++;
					}
				}
			}
			bCurrentGapIsUnbroken = true;
		}
		bPrevKeepFile = bKeepFile;
		if (bKeepFile)
		{
			LastKeepIndex = i;
		}
		PrevOrder = Order;
	}

#if GUARANTEE_UASSET_AND_UEXP_IN_SAME_PAK
	int64 OrphanedFilesSizeIncrease = 0;
	int32 OrphanedFileCount = AddOrphanedFiles(FilesToPak, FileSizes, UAssetToUexpMapping, IncludeFilesMask, OrphanedFilesSizeIncrease);
#endif

	int32 PatchFileCount = 0;
	int64 PatchSize = ComputePatchSize(FilesToPak, FileSizes, IncludeFilesMask, PatchFileCount);

	FString PrefixString = TEXT("");
	if (bFillPrimaryOrderFiles && !bFillSecondaryOrderFiles)
	{
		PrefixString = TEXT("[PRIMARY]");
	}
	else if (bFillSecondaryOrderFiles && !bFillPrimaryOrderFiles)
	{
		PrefixString = TEXT("[SECONDARY]");
	}
	if (PatchSize > MaxPatchSize && !bForce)
	{
		UE_LOG(LogPakFile, SEEK_OPT_VERBOSITY, TEXT("Patch seek optimization step %s - Gap size: %" INT64_FMT "KB - FAILED (patch too big)"), *PrefixString, MaxGapSizeBytes / 1024);
		return false;
	}

	// Stop if we didn't actually make patch size better
	float FragmentationPercentageAfterGapFill = GetFragmentationPercentage(FilesToPak, IncludeFilesMask, MaxAdjacentOrderDiff, true);
	if (FragmentationPercentageAfterGapFill >= FragmentationPercentageOriginal && !bForce)
	{
		UE_LOG(LogPakFile, SEEK_OPT_VERBOSITY, TEXT("Patch seek optimization step %s - Gap size: %" INT64_FMT "KB - FAILED (contiguous block count didn't improve)"), *PrefixString, MaxGapSizeBytes / 1024);
		return false;
	}
	InOutIncludeFilesMask = IncludeFilesMask;
	UE_LOG(LogPakFile, SEEK_OPT_VERBOSITY, TEXT("Patch seek optimization step %s - Gap size: %" INT64_FMT "KB - SUCCEEDED"), *PrefixString, MaxGapSizeBytes / 1024);
	return true;
}

bool DoIncrementalGapFilling(const TArray<FPakInputPair>& FilesToPak, const TArray<int64>& FileSizes, const TArray<int32>& UAssetToUexpMapping, TBitArray<>& IncludeFilesMask, int64 MinGapSize, int64 MaxGapSize, int64 MaxAdjacentOrderDiff, bool bFillPrimaryOrderFiles, bool bFillSecondaryOrderFiles, int64 MaxPatchSize)
{
	int64 GapSize = MinGapSize;
	bool bSuccess = false;
	while (GapSize <= MaxGapSize)
	{
		if (DoGapFillingIteration(FilesToPak, FileSizes, UAssetToUexpMapping, IncludeFilesMask, GapSize, MaxAdjacentOrderDiff, false, MaxPatchSize, bFillPrimaryOrderFiles, bFillSecondaryOrderFiles))
		{
			bSuccess = true;
		}
		else
		{
			// Try with 75% of the max gap size
			bSuccess |= DoGapFillingIteration(FilesToPak, FileSizes, UAssetToUexpMapping, IncludeFilesMask, (int64)((double)GapSize*0.75), MaxAdjacentOrderDiff, false, MaxPatchSize, bFillPrimaryOrderFiles, bFillSecondaryOrderFiles);
			break;
		}
		GapSize *= 2;
	}
	return bSuccess;
}

void ApplyGapFilling(const TArray<FPakInputPair>& FilesToPak, const TArray<int64>& FileSizes, const TArray<int32>& UAssetToUexpMapping, TBitArray<>& IncludeFilesMask, const FPatchSeekOptParams& SeekOptParams)
{
	UE_CLOGF(SeekOptParams.MaxGapSize == 0, LogPakFile, Fatal, "ApplyGapFilling requires MaxGapSize > 0");
	check(SeekOptParams.MaxGapSize > 0);
	float FragmentationPercentageOriginal = GetFragmentationPercentage(FilesToPak, IncludeFilesMask, SeekOptParams.MaxAdjacentOrderDiff, true);
	float FragmentationPercentageOriginalPrimary = GetFragmentationPercentage(FilesToPak, IncludeFilesMask, SeekOptParams.MaxAdjacentOrderDiff, false);
	int32 OriginalPatchFileCount = 0;
	int64 OriginalPatchSize = ComputePatchSize(FilesToPak, FileSizes, IncludeFilesMask, OriginalPatchFileCount);
	int64 IncrementalMaxPatchSize = int64(double(OriginalPatchSize) + double(OriginalPatchSize) * double(SeekOptParams.MaxInflationPercent) * 0.01);
	int64 MinIncrementalGapSize = 4 * 1024;

	ESeekOptMode SeekOptMode = SeekOptParams.Mode;
	switch (SeekOptMode)
	{
	case ESeekOptMode::OnePass:
	{
		DoGapFillingIteration(FilesToPak, FileSizes, UAssetToUexpMapping, IncludeFilesMask, SeekOptParams.MaxGapSize, SeekOptParams.MaxAdjacentOrderDiff, true);
	}
	break;
	case ESeekOptMode::Incremental:
	case ESeekOptMode::Incremental_OnlyPrimaryOrder:
	{
		UE_CLOGF(SeekOptParams.MaxInflationPercent == 0.0f, LogPakFile, Fatal, "ESeekOptMode::Incremental* requires MaxInflationPercent > 0.0");
		bool bFillSecondaryOrderFiles = (SeekOptParams.Mode == ESeekOptMode::Incremental);
		DoIncrementalGapFilling(FilesToPak, FileSizes, UAssetToUexpMapping, IncludeFilesMask, MinIncrementalGapSize, SeekOptParams.MaxGapSize, SeekOptParams.MaxAdjacentOrderDiff, true, bFillSecondaryOrderFiles, IncrementalMaxPatchSize);
	}
	break;
	case ESeekOptMode::Incremental_PrimaryThenSecondary:
	{
		UE_CLOGF(SeekOptParams.MaxInflationPercent == 0.0f, LogPakFile, Fatal, "ESeekOptMode::Incremental* requires MaxInflationPercent > 0.0");
		int64 PassMaxPatchSize[3];
		PassMaxPatchSize[0] = OriginalPatchSize + (int64)((double)(IncrementalMaxPatchSize - OriginalPatchSize) * 0.9);
		PassMaxPatchSize[1] = IncrementalMaxPatchSize;
		PassMaxPatchSize[2] = IncrementalMaxPatchSize;
		for (int32 i = 0; i < 3; i++)
		{
			bool bFillPrimaryOrderFiles = (i == 0) || (i == 2);
			bool bFillSecondaryOrderFiles = (i == 1);
			DoIncrementalGapFilling(FilesToPak, FileSizes, UAssetToUexpMapping, IncludeFilesMask, MinIncrementalGapSize, SeekOptParams.MaxGapSize, SeekOptParams.MaxAdjacentOrderDiff, bFillPrimaryOrderFiles, bFillSecondaryOrderFiles, PassMaxPatchSize[i]);
		}
	}
	break;
	}

	int32 NewPatchFileCount = 0;
	int64 NewPatchSize = ComputePatchSize(FilesToPak, FileSizes, IncludeFilesMask, NewPatchFileCount);

	double OriginalSizeMB = double(OriginalPatchSize) / 1024.0 / 1024.0;
	double SizeIncreaseMB = double(NewPatchSize - OriginalPatchSize) / 1024.0 / 1024.0;
	double TotalSizeMB = OriginalSizeMB + SizeIncreaseMB;
	double SizeIncreasePercent = 100.0 * SizeIncreaseMB / OriginalSizeMB;
	if (NewPatchFileCount == OriginalPatchSize)
	{
		UE_LOGF(LogPakFile, SEEK_OPT_VERBOSITY, "Patch seek optimization did not modify patch pak size (no additional files added)");
	}
	else
	{
		UE_LOGF(LogPakFile, SEEK_OPT_VERBOSITY, "Patch seek optimization increased estimated patch pak size from %.2fMB to %.2fMB (+%.1f%%)", OriginalSizeMB, TotalSizeMB, SizeIncreasePercent);
		UE_LOGF(LogPakFile, SEEK_OPT_VERBOSITY, "Total files added : %d (from %d)", NewPatchFileCount - OriginalPatchFileCount, OriginalPatchFileCount);
	}


	float FragmentationPercentageNew = GetFragmentationPercentage(FilesToPak, IncludeFilesMask, SeekOptParams.MaxAdjacentOrderDiff, true);
	float FragmentationPercentageNewPrimary = GetFragmentationPercentage(FilesToPak, IncludeFilesMask, SeekOptParams.MaxAdjacentOrderDiff, false);
	UE_LOGF(LogPakFile, SEEK_OPT_VERBOSITY, "Fragmentation pre-optimization: %.2f%%", FragmentationPercentageOriginal);
	UE_LOGF(LogPakFile, SEEK_OPT_VERBOSITY, "Fragmentation final: %.2f%%", FragmentationPercentageNew);
	UE_LOGF(LogPakFile, SEEK_OPT_VERBOSITY, "Fragmentation pre-optimization (primary files): %.2f%%", FragmentationPercentageOriginalPrimary);
	UE_LOGF(LogPakFile, SEEK_OPT_VERBOSITY, "Fragmentation final (primary files): %.2f%%", FragmentationPercentageNewPrimary);
}

void RemoveIdenticalFiles( TArray<FPakInputPair>& FilesToPak, const FString& SourceDirectory, const TMap<FString, FFileInfo>& FileHashes, const FPatchSeekOptParams& SeekOptParams, const FString ChangedFilesOutputFilename)
{
	FArchive* ChangedFilesArchive = nullptr;
	if (ChangedFilesOutputFilename.IsEmpty() == false)
	{
		ChangedFilesArchive = IFileManager::Get().CreateFileWriter(*ChangedFilesOutputFilename);
	}

	FString HashFilename = SourceDirectory / TEXT("Hashes.txt");

	if (IFileManager::Get().FileExists(*HashFilename) )
	{
		FString EntireFile;
		FFileHelper::LoadFileToString(EntireFile, *HashFilename);
	}

	TBitArray<> IncludeFilesMask;
	IncludeFilesMask.Add(true, FilesToPak.Num());

	TMap<FString, int32> SourceFileToIndex;
	for (int i = 0; i < FilesToPak.Num(); i++)
	{
		SourceFileToIndex.Add(FilesToPak[i].Source, i);
	}

	// Generate the index mapping from UExp to corresponding UAsset (and vice versa)
	TArray<int32> UAssetToUexpMapping;
	UAssetToUexpMapping.Empty(FilesToPak.Num());
	for (int i = 0; i<FilesToPak.Num(); i++)
	{
		UAssetToUexpMapping.Add(-1);
	}

	ParallelFor(FilesToPak.Num(), [&FilesToPak, &SourceFileToIndex, &UAssetToUexpMapping](int32 Idx)
		{
			const auto& NewFile = FilesToPak[Idx];
			FString Ext(FPaths::GetExtension(FilesToPak[Idx].Source));
			if (Ext.Equals("uasset", ESearchCase::IgnoreCase) || Ext.Equals("umap", ESearchCase::IgnoreCase))
			{
				FString UexpDestFilename = FPaths::ChangeExtension(NewFile.Source, "uexp");
				int32* UexpIndexPtr = SourceFileToIndex.Find(UexpDestFilename);
				if (UexpIndexPtr)
				{
					UAssetToUexpMapping[*UexpIndexPtr] = Idx;
					UAssetToUexpMapping[Idx] = *UexpIndexPtr;
				}
			}
		});
	TArray<int64> FileSizes;
	FileSizes.AddDefaulted(FilesToPak.Num());

	TArray<FString> ChangedFilesArchiveArray;
	ChangedFilesArchiveArray.SetNum(FilesToPak.Num());
    // Mark files to remove if they're unchanged
	ParallelFor(FilesToPak.Num(), [SourceDirectory, ChangedFilesArchive, &FilesToPak, &FileHashes, &FileSizes, &IncludeFilesMask, &ChangedFilesArchiveArray](int32 Idx)
		{
			const auto& NewFile = FilesToPak[Idx];
			if (NewFile.bIsDeleteRecord)
			{
				return;
			}
			FString SourceFileNoMountPoint = NewFile.Dest.Replace(TEXT("../../../"), TEXT(""));
			FString SourceFilename = SourceDirectory / SourceFileNoMountPoint;

			const FFileInfo* FoundFileHash = FileHashes.Find(SourceFileNoMountPoint);
			if (!FoundFileHash)
			{
				FoundFileHash = FileHashes.Find(NewFile.Dest);
			}

			if (!FoundFileHash)
			{
				UE_LOGF(LogPakFile, Display, "Didn't find hash for %ls No mount %ls", *SourceFilename, *SourceFileNoMountPoint);
			}

#if GUARANTEE_UASSET_AND_UEXP_IN_SAME_PAK
			// uexp files are always handled with their corresponding uasset file
			bool bForceInclude = false;
			if (!FPaths::GetExtension(SourceFilename).Equals("uexp", ESearchCase::IgnoreCase))
			{
				bForceInclude = FoundFileHash && FoundFileHash->bForceInclude;
			}
#else
			bool bForceInclude = FoundFileHash && FoundFileHash->bForceInclude;
#endif

			FString DestFilename = NewFile.Source;
			bool bFileIsIdentical = FileIsIdentical(SourceFilename, DestFilename, FoundFileHash, &FileSizes[Idx]);

			if (ChangedFilesArchive && !bFileIsIdentical)
			{
				ChangedFilesArchiveArray[Idx] = DestFilename;
			}

			if (!bForceInclude && bFileIsIdentical)
			{
				UE_LOGF(LogPakFile, Display, "Source file %ls matches dest file %ls and will not be included in patch", *SourceFilename, *DestFilename);
				// remove from the files to pak list
				IncludeFilesMask[Idx] = false;
			}
		});
	if (ChangedFilesArchive)
	{
		for (const FString& Log : ChangedFilesArchiveArray)
		{
			if (!Log.IsEmpty())
			{
				ChangedFilesArchive->Logf(TEXT("%s\n"), *Log);
			}
		}
	}

	// Add corresponding UExp/UBulk files to the patch if one is included but not the other (uassets and uexp files must be in the same pak)
#if GUARANTEE_UASSET_AND_UEXP_IN_SAME_PAK
	ParallelFor(FilesToPak.Num(), [&FilesToPak, &UAssetToUexpMapping, &IncludeFilesMask](int32 Idx)
		{
			int32 CounterpartFileIndex = UAssetToUexpMapping[Idx];
			if (CounterpartFileIndex != -1)
			{
				if (IncludeFilesMask[Idx] != IncludeFilesMask[CounterpartFileIndex])
				{
					UE_LOGF(LogPakFile, Display, "One of %ls and %ls is different from source, so both will be included in patch", *FilesToPak[Idx].Source, *FilesToPak[CounterpartFileIndex].Source);
					IncludeFilesMask[Idx] = true;
					IncludeFilesMask[CounterpartFileIndex] = true;
				}
			}
		});
#endif

	if (SeekOptParams.Mode != ESeekOptMode::None)
	{
		ApplyGapFilling(FilesToPak, FileSizes, UAssetToUexpMapping, IncludeFilesMask, SeekOptParams);
	}

	// Compress the array while preserving the order, removing the files we marked to remove
	int32 WriteIndex = 0;
	for ( int ReadIndex=0; ReadIndex<IncludeFilesMask.Num(); ReadIndex++)
	{
		if (IncludeFilesMask[ReadIndex])
		{
			FilesToPak[WriteIndex++] = FilesToPak[ReadIndex];
		}
	}
	int NumToRemove = FilesToPak.Num() - WriteIndex;
	FilesToPak.RemoveAt(WriteIndex, NumToRemove, EAllowShrinking::Yes);

	if (ChangedFilesArchive)
	{
		ChangedFilesArchive->Close();
		delete ChangedFilesArchive;
		ChangedFilesArchive = nullptr;
	}
}

void ProcessLegacyFileMoves( TArray<FPakInputPair>& InDeleteRecords, TMap<FString, FFileInfo>& InExistingPackagedFileHashes, const FString& InInputPath, const TArray<FPakInputPair>& InFilesToPak, int32 CurrentPatchChunkIndex )
{
	double StartTime = FPlatformTime::Seconds();


	TArray<FString> PakFileList;
	IFileManager::Get().FindFiles(PakFileList, *InInputPath, TEXT(".pak") );
	if( PakFileList.Num() == 0 )
	{
		UE_LOGF( LogPakFile, Error, "No pak files searching \"%ls\"", *InInputPath );
		return;
	}

	struct FFileChunkRevisionInfo
	{
		FString PakFilename;
		int32 PakPriority;
		int32 PakChunkIndex;
		int32 PakVersion;
	};
	TMap<FString, FFileChunkRevisionInfo> DeletedFileRevisions;
	TMap<FString, FFileChunkRevisionInfo> RequiredFileRevisions;

	TSet<FString> DeleteRecordSourceNames;
	for (const FPakInputPair& DeleteRecord : InDeleteRecords)
	{
		DeleteRecordSourceNames.Add(DeleteRecord.Source);
	}

	TSet<FString> FilesToPakDestNames;
	for (const FPakInputPair& FileToPak : InFilesToPak)
	{
		FilesToPakDestNames.Add(FileToPak.Dest);
	}

	for (int32 PakFileIndex = 0; PakFileIndex < PakFileList.Num(); PakFileIndex++)
	{
		FString PakFilename = InInputPath + "\\" + PakFileList[PakFileIndex];
		int32 PakPriority = GetPakPriorityFromFilename(PakFilename);
		int32 PakChunkIndex = GetPakChunkIndexFromFilename(PakFilename);

		UE_LOGF(LogPakFile, Display, "Checking old pak file \"%ls\" Pri:%d Chunk:%d.", *PakFilename, PakPriority, PakChunkIndex );


		TRefCountPtr<FPakFile> PakFilePtr = MakeRefCount<FPakFile>(&FPlatformFileManager::Get().GetPlatformFile(), *PakFilename, /*bIsSigned=*/ false);
		FPakFile& PakFile = *PakFilePtr;
		if (PakFile.IsValid())
		{
			FString PakMountPoint = PakFile.GetMountPoint().Replace(TEXT("../../../"), TEXT(""));

			const bool bIncludeDeleted = true;
			if (!PakFile.HasFilenames())
			{
				UE_LOGF(LogPakFile, Error, "Pakfiles were loaded without Filenames, cannot ProcessLegacyFileMoves.");
				return;
			}
			for (FPakFile::FPakEntryIterator It(PakFile,bIncludeDeleted); It; ++It)
			{
				FString AssetName = PakMountPoint;
				if (!AssetName.IsEmpty() && !AssetName.EndsWith("/"))
				{
					AssetName += "/";
				}
				AssetName += *It.TryGetFilename();

				bool bHasNewDeleteRecord = DeleteRecordSourceNames.Contains(AssetName);

				FFileChunkRevisionInfo Revision;
				Revision.PakFilename = PakFileList[PakFileIndex];
				Revision.PakPriority = PakPriority;
				Revision.PakChunkIndex = PakChunkIndex;
				Revision.PakVersion = PakFile.GetInfo().Version;

				TMap<FString, FFileChunkRevisionInfo>* DestList = nullptr;

				if( bHasNewDeleteRecord )
				{
					DestList = &DeletedFileRevisions;
				}
				else if( InExistingPackagedFileHashes.Contains(AssetName) )
				{
					FString DestAssetName = TEXT("../../../") + AssetName;
					bool bRequiredFile = FilesToPakDestNames.Contains(DestAssetName);

					if(bRequiredFile)
					{
						DestList = &RequiredFileRevisions;
					}
				}

				if( DestList != nullptr )
				{
					if( !DestList->Contains(AssetName) )
					{
						DestList->Add(AssetName,Revision);
					}
					else if( (*DestList)[AssetName].PakPriority < PakPriority )
					{
						(*DestList)[AssetName] = Revision;
					}
				}

			}
		}
	}

	//prevent delete records being created for files that have historically been moved
	for (auto Itr : DeletedFileRevisions)
	{
		UE_LOGF(LogPakFile, Display, "checking deleted revision %ls chunk %d vs %d   pak version %d vs %d", *Itr.Key, Itr.Value.PakChunkIndex, CurrentPatchChunkIndex, Itr.Value.PakVersion, FPakInfo::PakFile_Version_DeleteRecords );

		//asset hasn't been deleted in the latest version and the latest known version is in a different chunk to us from a previous version of unrealpak
		if( Itr.Value.PakChunkIndex != CurrentPatchChunkIndex )
		{
			int NumDeleted = InDeleteRecords.RemoveAll( [&]( const FPakInputPair& InPair )
			{
				return InPair.Source == Itr.Key;
			});
			if( NumDeleted > 0 )
			{
				UE_LOGF( LogPakFile, Display, "Ignoring delete record for %ls - it was moved to %ls before delete records were created", *Itr.Key, *FPaths::GetCleanFilename(Itr.Value.PakFilename) );
			}
		}
	}

	//make sure files who's latest revision was in a different chunk to the one we're building are added to the pak
	//#TODO: I think this RequiredFileRevision code is not needed
	for (auto Itr : RequiredFileRevisions)
	{
		if (Itr.Value.PakVersion < FPakInfo::PakFile_Version_DeleteRecords && Itr.Value.PakChunkIndex != CurrentPatchChunkIndex )
		{
			if( InExistingPackagedFileHashes.Contains(Itr.Key) )
			{
				UE_LOGF( LogPakFile, Display, "Ensuring %ls is included in the pak file - it was moved to %ls before delete records were created", *Itr.Key, *FPaths::GetCleanFilename(Itr.Value.PakFilename) );
				InExistingPackagedFileHashes[Itr.Key].bForceInclude = true;
			}
		}
	}

	UE_LOGF(LogPakFile, Display, "...took %.2fs to manage legacy patch pak files", FPlatformTime::Seconds() - StartTime );
}


TArray<FPakInputPair> GetNewDeleteRecords( const TArray<FPakInputPair>& InFilesToPak, const TMap<FString, FFileInfo>& InExistingPackagedFileHashes)
{
	double StartTime = FPlatformTime::Seconds();
	TArray<FPakInputPair> DeleteRecords;

	//build lookup table of files to pack
	TSet<FString> FilesToPack;
	for (const FPakInputPair& PakEntry : InFilesToPak)
	{
		FString PakFilename = PakEntry.Dest.Replace(TEXT("../../../"), TEXT(""));
		FilesToPack.Add(PakFilename);
	}

	//check all assets in the previous patch packs
	for (const TTuple<FString, FFileInfo>& Pair : InExistingPackagedFileHashes)
	{
		//ignore this file if the most recent revision is deleted already
		if (Pair.Value.bIsDeleteRecord)
		{
			continue;
		}

		//see if the file exists in the files to package
		FString SourceFileName = Pair.Key;
		bool bFound = FilesToPack.Contains(SourceFileName);

		if (bFound == false)
		{
			//file cannot be found now, and was not deleted in the most recent pak patch
			FPakInputPair DeleteRecord;
			DeleteRecord.bIsDeleteRecord = true;
			DeleteRecord.Source = SourceFileName;
			DeleteRecord.Dest = TEXT("../../../") + SourceFileName;
			DeleteRecords.Add(DeleteRecord);
			UE_LOGF(LogPakFile, Display, "Existing pak entry %ls not found in new pak asset list, so a delete record will be created in the patch pak.", *SourceFileName);
		}
 	}


	UE_LOGF(LogPakFile, Display, "Took %.2fS for delete records", FPlatformTime::Seconds()-StartTime );
	return DeleteRecords;
}

bool AtLeastOnePakFileExists(const FString& PakFilename)
{
	if (IFileManager::Get().FileExists(*PakFilename))
	{
		return true;
	}
	else
	{
		// If file doesn't exist try using it as a search string, it may contain wild cards
		TArray<FString> PakFileList;
		FString PakFileDirectory = FPaths::GetPath(PakFilename);
		IFileManager::Get().FindFiles(PakFileList, *PakFileDirectory, *FPaths::GetCleanFilename(PakFilename));

		return PakFileList.Num() > 0;
	}
}

FString GetPakPath(const TCHAR* SpecifiedPath, bool bIsForCreation)
{
	FString PakFilename(SpecifiedPath);
	FPaths::MakeStandardFilename(PakFilename);
	
	// if we are trying to open (not create) it, but BaseDir relative doesn't exist, look in LaunchDir
	if (!bIsForCreation && !AtLeastOnePakFileExists(PakFilename))
	{
		PakFilename = FPaths::LaunchDir() + SpecifiedPath;

		if (!AtLeastOnePakFileExists(PakFilename))
		{
			UE_LOGF(LogPakFile, Fatal, "Existing pak file %ls could not be found (checked against binary and launch directories)", SpecifiedPath);
			return TEXT("");
		}
	}
	
	return PakFilename;
}

bool Repack(const FString& InputPakFile, const FString& OutputPakFile, const FPakCommandLineParameters& CmdLineParameters, const FKeyChain& InKeyChain, bool bIncludeDeleted)
{
	bool bResult = false;

	// Extract the existing pak file
	TMap<FString, FFileInfo> Hashes;
	TArray<FPakInputPair> Entries;
	TArray<FPakInputPair> DeletedEntries;
	FPakOrderMap OrderMap;
	TArray<FGuid> EncryptionKeys;
	bool bAnySigned = false;
	FString TempDir = FPaths::EngineIntermediateDir() / TEXT("UnrealPak") / TEXT("Repack") / FPaths::GetBaseFilename(InputPakFile);
	if (ExtractFilesFromPak(*InputPakFile, Hashes, *TempDir, false, InKeyChain, nullptr, &Entries, &DeletedEntries, &OrderMap, &EncryptionKeys, &bAnySigned))
	{
		TArray<FPakInputPair> FilesToAdd;
		CollectFilesToAdd(FilesToAdd, Entries, OrderMap, CmdLineParameters);

		if (bIncludeDeleted)
		{
			for( const FPakInputPair& Entry : DeletedEntries )
			{
				FilesToAdd.Add(Entry);
			}
		}
		else if (DeletedEntries.Num() > 0)
		{
			UE_LOGF(LogPakFile, Display, "%ls has %d delete records - these will not be included in the repackage. Specify -IncludeDeleted to include them", *InputPakFile, DeletedEntries.Num() );
		}

		// Get a temporary output filename. We'll only create/replace the final output file once successful.
		FString TempOutputPakFile = FPaths::CreateTempFilename(*FPaths::GetPath(OutputPakFile), *FPaths::GetCleanFilename(OutputPakFile));

		FPakCommandLineParameters ModifiedCmdLineParameters = CmdLineParameters;
		ModifiedCmdLineParameters.bSign = bAnySigned && (InKeyChain.GetSigningKey() != InvalidRSAKeyHandle);
		
		FKeyChain ModifiedKeyChain = InKeyChain;
		ModifiedKeyChain.SetPrincipalEncryptionKey( InKeyChain.GetEncryptionKeys().Find(EncryptionKeys.Num() ? EncryptionKeys[0] : FGuid()) );

		// Create the new pak file
		UE_LOGF(LogPakFile, Display, "Creating %ls...", *OutputPakFile);
		if (CreatePakFile(*TempOutputPakFile, FilesToAdd, ModifiedCmdLineParameters, ModifiedKeyChain))
		{
			IFileManager::Get().Move(*OutputPakFile, *TempOutputPakFile);

			FString OutputSigFile = FPaths::ChangeExtension(OutputPakFile, TEXT(".sig"));
			if (IFileManager::Get().FileExists(*OutputSigFile))
			{
				IFileManager::Get().Delete(*OutputSigFile);
			}

			FString TempOutputSigFile = FPaths::ChangeExtension(TempOutputPakFile, TEXT(".sig"));
			if (IFileManager::Get().FileExists(*TempOutputSigFile))
			{
				IFileManager::Get().Move(*OutputSigFile, *TempOutputSigFile);
			}

			bResult = true;
		}
	}
	IFileManager::Get().DeleteDirectory(*TempDir, false, true);

	return bResult;
}



int32 NumberOfWorkerThreadsDesired()
{
	const int32 MaxThreads = 64;
	const int32 NumberOfCores = FPlatformMisc::NumberOfCoresIncludingHyperthreads();
	// need to spawn at least one worker thread (see FTaskGraphImplementation)
	return FMath::Max(FMath::Min(NumberOfCores - 1, MaxThreads), 1);
}

void CheckAndReallocThreadPool()
{
	if (FPlatformProcess::SupportsMultithreading())
	{
		const int32 ThreadsSpawned = GThreadPool->GetNumThreads();
		const int32 DesiredThreadCount = NumberOfWorkerThreadsDesired();
		if (ThreadsSpawned < DesiredThreadCount)
		{
			UE_LOGF(LogPakFile, Log, "Engine only spawned %d worker threads, bumping up to %d!", ThreadsSpawned, DesiredThreadCount);
			GThreadPool->Destroy();
			GThreadPool = FQueuedThreadPool::Allocate();
			verify(GThreadPool->Create(DesiredThreadCount, 128 * 1024));
		}
		else
		{
			UE_LOGF(LogPakFile, Log, "Continuing with %d spawned worker threads.", ThreadsSpawned);
		}
	}
}

bool MakeBinaryConfig(const TCHAR* CmdLine)
{
	FString OutputFile;
	if (!FParse::Value(CmdLine, TEXT("OutputFile="), OutputFile))
	{
		UE_LOGF(LogMakeBinaryConfig, Error, "OutputFile= parameter required");
		return false;
	}

	FString StagedPluginsFile;
	if (!FParse::Value(CmdLine, TEXT("StagedPluginsFile="), StagedPluginsFile))
	{
		UE_LOGF(LogMakeBinaryConfig, Error, "StagedPluginsFile= parameter required");
		return false;
	}

	FString ProjectFile;
	if (!FParse::Value(CmdLine, TEXT("Project="), ProjectFile))
	{
		UE_LOGF(LogMakeBinaryConfig, Error, "ProjectFile= parameter required (path to .uproject file)");
		return false;
	}

	FString PlatformName;
	if (!FParse::Value(CmdLine, TEXT("Platform="), PlatformName))
	{
		UE_LOGF(LogMakeBinaryConfig, Error, "Platform= parameter required (Ini platform name, not targetplatform name - Windows, not Win64 or WindowsClient)");
		return false;
	}

	FString ProjectDir = FPaths::GetPath(ProjectFile);
	ProjectDir = ProjectDir.Replace(TEXT("\\"), TEXT("/"));
	ProjectDir = ProjectDir.Replace(*FPaths::RootDir(), TEXT("../../../"));

	FConfigCacheIni Config(EConfigCacheType::Temporary, *PlatformName);
	FConfigContext Context = FConfigContext::ReadIntoConfigSystem(&Config, PlatformName);
	Context.ProjectConfigDir = FPaths::Combine(ProjectDir, TEXT("Config/"));
	Context.bIsMakingBinaryConfig = true;

	auto LoadIniCache = [] (FConfigCacheIni& Config, const TArray<FString>&PluginAndConfigsList, bool bUseLocalPaths)
	{
		// reset any current cache
		delete Config.StagedGlobalConfigCache;
		Config.StagedGlobalConfigCache = nullptr;
		Config.StagedPluginConfigCache.Empty();

		FName CurrentPlugin = NAME_None;
		for (const FString& Line : PluginAndConfigsList)
		{
			// indented lines are staged paths to config files
			if (Line.StartsWith(TEXT("  ")))
			{
				// a line looks like: "  D:\UE\Engine\Foo.png|Engine/Foo.png"
				int PipeLoc;
				Line.FindChar('|', PipeLoc);

				FString CachedPath;
				if (bUseLocalPaths)
				{
					CachedPath = Line.Mid(2, PipeLoc - 2);
					// must be relative so that the config discovery code matches paths correctly
					CachedPath = CachedPath.Replace(TEXT("\\"), TEXT("/"));
					CachedPath = CachedPath.Replace(*FPaths::RootDir(), TEXT("../../../"));
				}
				else
				{
					CachedPath = FString(TEXT("../../../")) + Line.Mid(PipeLoc + 1);
				}

				// the first set of lines are configs without a plugin, so those are global
				if (CurrentPlugin == NAME_None)
				{
					if (Config.StagedGlobalConfigCache == nullptr)
					{
						Config.StagedGlobalConfigCache = new TSet<FString>();
					}
					Config.StagedGlobalConfigCache->Add(CachedPath);
				}
				else
				{
					Config.StagedPluginConfigCache[CurrentPlugin].Add(CachedPath);
				}
			}
			else
			{
				CurrentPlugin = *Line;
				Config.StagedPluginConfigCache.FindOrAdd(CurrentPlugin);
			}
		}
	};

	// read in the cache of plugin ini files - 
	TArray<FString> PluginAndConfigsList;
	if (!FFileHelper::LoadFileToStringArray(PluginAndConfigsList, *StagedPluginsFile))
	{
		UE_LOGF(LogMakeBinaryConfig, Fatal, "Failed to load required file %ls", *StagedPluginsFile);
	}

	// IMPORTANT - we use local paths to config files when reading in the known configs to limit
	// the binary config to only contain the staged ini files (ie Restricted files that exist when 
	// building the binary config, but aren't staged, and shouldn't be in the binary)
	LoadIniCache(Config, PluginAndConfigsList, true);

	FString ScalabilityPlatformOverride;
	FParse::Value(FCommandLine::Get(), TEXT("ScalabilityIniPlatformOverride="), ScalabilityPlatformOverride);
	if (!ScalabilityPlatformOverride.IsEmpty())
	{
		FString EngineConfigPlatformOverride = FString::Format(TEXT("{0}{1}/{1}Scalability.ini"), { *Context.EngineConfigDir, *ScalabilityPlatformOverride });
		FString ProjectConfigPlatformOverride = FString::Format(TEXT("{0}{1}/{1}Scalability.ini"), { *Context.ProjectConfigDir, *ScalabilityPlatformOverride });
		if (Config.StagedGlobalConfigCache == nullptr)
		{
			Config.StagedGlobalConfigCache = new TSet<FString>();
		}
		Config.StagedGlobalConfigCache->Add(EngineConfigPlatformOverride);
		Config.StagedGlobalConfigCache->Add(ProjectConfigPlatformOverride);
	}
	Config.InitializeKnownConfigFiles(Context);

	// then we need to save out with staged paths so that runtime can know what files exist in the cache
	LoadIniCache(Config, PluginAndConfigsList, false);
	Config.bIsReadyForUse = true;

	// pull out deny list entries

	TArray<FString> KeyDenyListStrings;
	TArray<FString> SectionsDenyList;
	Config.GetArray(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("IniKeyDenylist"), KeyDenyListStrings, GGameIni);
	Config.GetArray(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("IniSectionDenylist"), SectionsDenyList, GGameIni);
	TArray<FName> KeysDenyList;
	for (const FString& Key : KeyDenyListStrings)
	{
		KeysDenyList.Add(FName(*Key));
	}

	for (const FString& Filename : Config.GetFilenames())
	{
		FConfigBranch* Branch = Config.FindBranch(*Filename, Filename);
		Branch->RunOnEachFile([&SectionsDenyList, &KeysDenyList](FConfigFile& File, const FString& Name)
			{
				for (const FString& Section : SectionsDenyList)
				{
					File.Remove(Section);
				}

				// now go over any remaining sections and remove keys
				for (const TPair<FString, FConfigSection>& SectionPair : (const FConfigFile&)File)
				{
					for (FName Key : KeysDenyList)
					{
						File.RemoveKeyFromSection(*SectionPair.Key, Key);
					}
				}
			});

		Branch->RunOnEachCommandStream([&SectionsDenyList, &KeysDenyList](FConfigCommandStream& Stream, const FString& Name)
			{
				for (const FString& Section : SectionsDenyList)
				{
					Stream.Remove(Section);
				}

				// now go over any remaining sections and remove keys
				for (TPair<FString, FConfigCommandStreamSection>& SectionPair : Stream)
				{
					for (FName Key : KeysDenyList)
					{
						SectionPair.Value.Remove(Key);
					}
				}
			});
	}

	// check the deny list removed itself
	KeyDenyListStrings.Empty();
	Config.GetArray(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("IniKeyDenylist"), KeyDenyListStrings, GGameIni);
	check(KeyDenyListStrings.Num() == 0);

	// allow delegates to modify the config data with some tagged binary data
	FCoreDelegates::FExtraBinaryConfigData ExtraData(Config, true);
	FCoreDelegates::TSAccessExtraBinaryConfigData().Broadcast(ExtraData);

	// write it all out!
	TArray<uint8> FileContent;
	{
		// Use FMemoryWriter because FileManager::CreateFileWriter doesn't serialize FName as string and is not overridable
		FMemoryWriter MemoryWriter(FileContent, true);

		Config.Serialize(MemoryWriter);
		MemoryWriter << ExtraData.Data;
	}

	if (!FFileHelper::SaveArrayToFile(FileContent, *OutputFile))
	{
		UE_LOGF(LogMakeBinaryConfig, Error, "Failed to create binary config file '%ls'", *OutputFile);
		return false;
	}

	return true;
}


/**
 * Application entry point
 * Params:
 *   -Test test if the pak file is healthy
 *   -Extract extracts pak file contents (followed by a path, i.e.: -extract D:\ExtractedPak)
 *   -Create=filename response file to create a pak file with
 *
 * @param	ArgC	Command-line argument count
 * @param	ArgV	Argument strings
 */
bool ExecuteUnrealPak(const TCHAR* CmdLine)
{
	UE_LOGF(LogPakFile, Display, "ProjectDir: %ls", *FPaths::ProjectDir());
	{
		FString IoStoreArg;
		if (FParse::Value(CmdLine, TEXT("-CreateGlobalContainer="), IoStoreArg) ||
			FParse::Value(CmdLine, TEXT("-CreateDLCContainer="), IoStoreArg))
		{
			return CreateIoStoreContainerFiles(CmdLine) == 0;
		}

		if (FParse::Value(CmdLine, TEXT("-ListContainer="), IoStoreArg))
		{
			return ListIoStoreContainer(CmdLine);
		}

		if (FParse::Value(CmdLine, TEXT("-ListContainerBulkData="), IoStoreArg))
		{
			return ListIoStoreContainerBulkData(CmdLine);
		}

		if (FParse::Value(CmdLine, TEXT("-ListContainerMetadataFile="), IoStoreArg))
		{
			return ListIoStoreContainerMetadataFile(CmdLine);
		}

		if (FParse::Value(CmdLine, TEXT("-DiffContainer="), IoStoreArg))
		{
			return DiffIoStoreContainer(CmdLine);
		}
	}

	// Parse all the non-option arguments from the command line
	TArray<FString> NonOptionArguments;
	for (const TCHAR* CmdLineEnd = CmdLine; *CmdLineEnd != 0;)
	{
		FString Argument = FParse::Token(CmdLineEnd, false);
		if (Argument.Len() > 0 && !Argument.StartsWith(TEXT("-")))
		{
			NonOptionArguments.Add(Argument);
		}
	}

	if (NonOptionArguments.Num() && NonOptionArguments[0] == TEXT("IoStore"))
	{
		return CreateIoStoreContainerFiles(CmdLine) == 0;
	}

	if (NonOptionArguments.Num() && NonOptionArguments[0] == TEXT("MakeBinaryConfig"))
	{
		return MakeBinaryConfig(CmdLine);
	}

	FString ProjectArg;
	if (FParse::Value(CmdLine, TEXT("-Project="), ProjectArg))
	{
		if (!IFileManager::Get().FileExists(*ProjectArg))
		{
			UE_LOGF(LogPakFile, Error, "Project file does not exist: %ls", *ProjectArg);
		}

		UE_LOGF(LogPakFile, Error, "Project should be specified as a first unnamed argument. E.g. 'UnrealPak %ls -arg1 -arg2", *ProjectArg);
		return false;
	}

	if (FParse::Param(CmdLine, TEXT("listformats")))
	{
		TArray<ICompressionFormat*> Formats = IModularFeatures::Get().GetModularFeatureImplementations<ICompressionFormat>(COMPRESSION_FORMAT_FEATURE_NAME);
		UE_LOGF(LogPakFile, Display, "Supported Pak Formats:");
		for (auto& Format : Formats)
		{
			UE_LOGF(LogPakFile, Display, "\t%ls %d", *Format->GetCompressionFormatName().ToString(), Format->GetVersion());
		}
		return true;
	}
	
	FString BatchFileName;
	if (FParse::Value(CmdLine, TEXT("-Batch="), BatchFileName))
	{
		TArray<FString> Commands;
		if (!FFileHelper::LoadFileToStringArray(Commands, *BatchFileName))
		{
			UE_LOGF(LogPakFile, Error, "Unable to read '%ls'", *BatchFileName);
			return false;
		}

		UE_LOGF(LogPakFile, Display, "Running UnrealPak in batch mode with commands:");
		for (int i = 0; i < Commands.Num(); i++)
		{
			UE_LOGF(LogPakFile, Display, "[%d] : %ls", i, *Commands[i]);
		}

		TAtomic<bool> Result(true);
		ParallelFor(Commands.Num(), [&Commands, &Result](int32 Idx) { if (!ExecuteUnrealPak(*Commands[Idx])) { Result = false; } });
		return Result;
	}


	FKeyChain KeyChain;
	LoadKeyChain(CmdLine, KeyChain);
	KeyChainUtilities::ApplyEncryptionKeys(KeyChain);

	bool IsTestCommand = FParse::Param(CmdLine, TEXT("Test"));
	bool IsVerifyCommand = FParse::Param(CmdLine, TEXT("Verify"));
	if (IsTestCommand || IsVerifyCommand)
	{
		if (NonOptionArguments.Num() != 1)
		{
			UE_LOGF(LogPakFile, Error, "Incorrect arguments. Expected: -Test <PakFile>");
			return false;
		}

		FString PakFilename = GetPakPath(*NonOptionArguments[0], false);
		return TestPakFile(*PakFilename, IsVerifyCommand);
	}

	if (FParse::Param(CmdLine, TEXT("List")))
	{
		TArray<FPakInputPair> Entries;
		FPakCommandLineParameters CmdLineParameters;
		ProcessCommandLine(CmdLine, NonOptionArguments, Entries, CmdLineParameters);

		if (NonOptionArguments.Num() != 1)
		{
			UE_LOGF(LogPakFile, Error, "Incorrect arguments. Expected: -List <PakFile> [-SizeFilter=N]");
			return false;
		}

		int64 SizeFilter = 0;
		FParse::Value(CmdLine, TEXT("SizeFilter="), SizeFilter);

		bool bExcludeDeleted = FParse::Param( CmdLine, TEXT("ExcludeDeleted") );

		// Handle if we're trying to list an entire directory or anything with wildcards.
		// This gets a little weird because GetPakPath does a bunch of normalization, and this
		// is trying to somewhat match behavior in IostoreUtilities.cpp.
		FString RawFileName = NonOptionArguments[0];
		TArray<FString> FilesToList;
		if (IFileManager::Get().DirectoryExists(*RawFileName))
		{
			// List the whole directory.
			FString Directory = RawFileName;
			FPaths::NormalizeDirectoryName(Directory);

			TArray<FString> FoundPaks;
			IFileManager::Get().FindFiles(FoundPaks, *(Directory / TEXT("*.pak")), true, false);

			for (const FString& Filename : FoundPaks)
			{
				FilesToList.Emplace(Directory / Filename);
			}
		}
		else if (RawFileName.Contains("*"))
		{
			// Technically GetPakPath handles wildcards, but it also only returns a single filename (?),
			// so we now assume if they include a wildcard they want multiple listings.
			FString Directory = FPaths::GetPath(RawFileName);
			FPaths::NormalizeDirectoryName(Directory);

			TArray<FString> FoundPaks;
			IFileManager::Get().FindFiles(FoundPaks, *RawFileName, true, false);

			for (const FString& Filename : FoundPaks)
			{
				FilesToList.Emplace(Directory / Filename);
			}
		}
		else
		{
			// Otherwise, use the old system so we don't break anything.
			FilesToList.Emplace(GetPakPath(*NonOptionArguments[0], false));
		}
		
		FString CSVFilename;
		FParse::Value(CmdLine, TEXT("csv="), CSVFilename);

		const bool bOutputCSV = !CSVFilename.IsEmpty();

		// If CSV is a path, then dump multiple CSVs. If it's a single CSV, then append.
		bool bAppendCSV = false;
		if (bOutputCSV)
		{
			if (FilesToList.Num() > 1)
			{
				bAppendCSV = !IFileManager::Get().DirectoryExists(*CSVFilename);
			}
			else
			{
				// If it's a single file, then if it's a directory use the pak filename as a base
				if (IFileManager::Get().DirectoryExists(*CSVFilename))
				{
					CSVFilename = CSVFilename / FPaths::GetCleanFilename(FilesToList[0]) + TEXT(".csv");
				}
			}
		}

		bool bExtractToMountPoint = FParse::Param(CmdLine, TEXT("ExtractToMountPoint"));
		bool bIncludeFooter = FParse::Param(CmdLine, TEXT("IncludeFooters"));

		bool bAllSucceeded = true;
		TArray<bool> bSucceeded;
		bSucceeded.SetNum(FilesToList.Num());

		ParallelFor(FilesToList.Num(), [&FilesToList, 
										&CSVFilename, 
										&bOutputCSV, 
										&bAppendCSV, 
										&bExtractToMountPoint, 
										&KeyChain, 
										&bIncludeFooter, 
										&SizeFilter,
										&bExcludeDeleted,
										&bSucceeded](int32 Idx)
		{
			const FString& PakFilename = FilesToList[Idx];
			FString ThisPaksCSV = CSVFilename;
			if (bOutputCSV && !bAppendCSV && FilesToList.Num() > 1)
			{
				// It's a directory.
				ThisPaksCSV = CSVFilename / FPaths::GetCleanFilename(PakFilename) + TEXT(".csv");
			}

			bSucceeded[Idx] = ListFilesInPak(*PakFilename, SizeFilter, !bExcludeDeleted, *ThisPaksCSV, bExtractToMountPoint, KeyChain, bAppendCSV, bIncludeFooter);
		});

		for (const bool &bDidSucceed : bSucceeded)
		{
			bAllSucceeded &= bDidSucceed;
		}

		return bAllSucceeded;
	}

	if (FParse::Param(CmdLine, TEXT("Info")))
	{
		if (NonOptionArguments.Num() != 1)
		{
			UE_LOGF(LogPakFile, Error, "Incorrect arguments. Expected: -Info <PakFile>");
			return false;
		}

		FString PakFilename = GetPakPath(*NonOptionArguments[0], false);

		return DumpPakInfo(PakFilename, KeyChain);
	}

	if (FParse::Param(CmdLine, TEXT("Diff")))
	{
		if(NonOptionArguments.Num() != 2)
		{
			UE_LOGF(LogPakFile,Error,"Incorrect arguments. Expected: -Diff <PakFile1|PakPath1> <PakFile2|PakPath2> [-NoUniques] [-NoUniquesFile1] [-NoUniquesFile2] -CryptoKeys2=<cryptofile>");
			return false;
		}

		auto GetPakFiles = [](const FString& PakFileOrFolder, FString& OutFolder, TArray<FString>& OutFiles)
		{
			if (FPaths::DirectoryExists(PakFileOrFolder))
			{
				OutFolder = PakFileOrFolder;
				IFileManager::Get().FindFiles(OutFiles, *PakFileOrFolder, TEXT(".pak"));
			}
			else
			{
				FString PakFile = GetPakPath(*PakFileOrFolder, false);
				OutFolder = FPaths::GetPath(PakFile);
				OutFiles.Emplace(FPaths::GetCleanFilename(PakFile));
			}
		};
		FString SourceFolderName;
		FString TargetFolderName;
		TArray<FString> SourcePakFiles;
		TArray<FString> TargetPakFiles;
		GetPakFiles(NonOptionArguments[0], SourceFolderName, SourcePakFiles);
		GetPakFiles(NonOptionArguments[1], TargetFolderName, TargetPakFiles);

		// Allow the suppression of unique file logging for one or both files
		const bool bLogUniques = !FParse::Param(CmdLine, TEXT("nouniques"));
		const bool bLogUniques1 = bLogUniques && !FParse::Param(CmdLine, TEXT("nouniquesfile1"));
		const bool bLogUniques2 = bLogUniques && !FParse::Param(CmdLine, TEXT("nouniquesfile2"));

		FString CryptoKeysFilename2;
		FKeyChain KeyChain2;
		if (FParse::Value(FCommandLine::Get(), TEXT("CryptoKeys2="), CryptoKeysFilename2) ||
			FParse::Value(FCommandLine::Get(), TEXT("TargetCryptoKeys="), CryptoKeysFilename2))
		{
			KeyChainUtilities::LoadKeyChainFromFile(CryptoKeysFilename2, KeyChain2);
			KeyChainUtilities::ApplyEncryptionKeys(KeyChain2);
		}
		else
		{
			KeyChain2 = KeyChain;
		}

		TSet<FString> SourcePakFileSet;
		TSet<FString> TargetPakFileSet;
		SourcePakFileSet.Append(SourcePakFiles);
		TargetPakFileSet.Append(TargetPakFiles);

		UE_LOGF(LogPakFile, Display, "Diff Source Pak Folder: %ls - %d pak file(s)",
			*SourceFolderName, SourcePakFiles.Num());
		UE_LOGF(LogPakFile, Display, "Diff Target Pak Folder: %ls - %d pak file(s)",
			*TargetFolderName, TargetPakFiles.Num());

		for (int32 I = SourcePakFiles.Num() - 1; I >= 0; --I)
		{
			if (!TargetPakFileSet.Contains(SourcePakFiles[I]))
			{
				if (bLogUniques1)
				{
					UE_LOGF(LogPakFile, Display, "Source PakFile '%ls' is missing in target folder",
						*SourcePakFiles[I]);
				}
				SourcePakFiles.RemoveAtSwap(I, EAllowShrinking::No);
			}
		}
		if (bLogUniques2)
		{
			for (int32 I = TargetPakFiles.Num() - 1; I >= 0; --I)
			{
				if (!SourcePakFileSet.Contains(TargetPakFiles[I]))
				{
					UE_LOGF(LogPakFile, Display, "Target PakFile '%ls' is missing in source folder",
						*TargetPakFiles[I]);
				}
			}
		}

		bool Success = true;
		FString SourceFile;
		FString TargetFile;
		for (const FString& PakFile : SourcePakFiles)
		{
			SourceFile = SourceFolderName / PakFile;
			SourceFile.ReplaceInline(TEXT("/"), TEXT("\\"));
			TargetFile = TargetFolderName / PakFile;
			TargetFile.ReplaceInline(TEXT("/"), TEXT("\\"));
			UE_LOGF(LogPakFile, Display, "Diffing PakFile '%ls'", *PakFile);
			Success &= DiffFilesInPaks(SourceFile, TargetFile, bLogUniques1, bLogUniques2, KeyChain, KeyChain2);
		}
		return Success;
	}

	if (FParse::Param(CmdLine, TEXT("Extract")))
	{
		TArray<FPakInputPair> Entries;
		FPakCommandLineParameters CmdLineParameters;
		ProcessCommandLine(CmdLine, NonOptionArguments, Entries, CmdLineParameters);
	
		if (NonOptionArguments.Num() != 2)
		{
			UE_LOGF(LogPakFile, Error, "Incorrect arguments. Expected: -Extract <PakFile> <OutputPath> [-responsefile=<outputresponsefilename> -order=<outputordermap>]");
			return false;
		}

		FString PakFilename = GetPakPath(*NonOptionArguments[0], false);

		FString ResponseFileName;
		FString OrderMapFileName;
		bool bGenerateResponseFile = FParse::Value(CmdLine, TEXT("responseFile="), ResponseFileName);
		bool bGenerateOrderMap = FParse::Value(CmdLine, TEXT("outorder="), OrderMapFileName);
		bool bUseFilter = false;
		FString Filter;
		FString DestPath = NonOptionArguments[1];

		bUseFilter = FParse::Value(CmdLine, TEXT("Filter="), Filter);
		bool bExtractToMountPoint = FParse::Param(CmdLine, TEXT("ExtractToMountPoint"));
		TMap<FString, FFileInfo> EmptyMap;
		TArray<FPakInputPair> ResponseContent;
		TArray<FPakInputPair>* UsedResponseContent = nullptr;
		TArray<FPakInputPair> DeletedContent;
		TArray<FPakInputPair>* UsedDeletedContent = nullptr;
		if (bGenerateResponseFile)
		{
			UsedResponseContent = &ResponseContent;
			UsedDeletedContent = &DeletedContent;
		}
		FPakOrderMap OrderMap;
		FPakOrderMap* UsedOrderMap = nullptr;
		if (bGenerateOrderMap)
		{
			UsedOrderMap = &OrderMap;
		}
		if (ExtractFilesFromPak(*PakFilename, EmptyMap, *DestPath, bExtractToMountPoint, KeyChain, bUseFilter ? &Filter : nullptr, UsedResponseContent, UsedDeletedContent, UsedOrderMap) == false)
		{
			return false;
		}

		if (bGenerateResponseFile)
		{
			FArchive* ResponseArchive = IFileManager::Get().CreateFileWriter(*ResponseFileName);
			ResponseArchive->SetIsTextFormat(true);
			// generate a response file
			if (FParse::Param(CmdLine, TEXT("includedeleted")))
			{
				for (int32 I = 0; I < DeletedContent.Num(); ++I)
				{
					ResponseArchive->Logf(TEXT("%s -delete"), *DeletedContent[I].Dest);
				}
			}

			for (int32 I = 0; I < ResponseContent.Num(); ++I)
			{
				const FPakInputPair& Response = ResponseContent[I];
				FString Line = FString::Printf(TEXT("%s %s"), *Response.Source, *Response.Dest);
				if (Response.bNeedEncryption)
				{
					Line += " -encrypt";
				}
				if (Response.bNeedsCompression)
				{
					Line += " -compress";
				}
				ResponseArchive->Logf(TEXT("%s"), *Line);
			}
			ResponseArchive->Close();
			delete ResponseArchive;
		}
		if (bGenerateOrderMap)
		{
			FArchive* OutputFile = IFileManager::Get().CreateFileWriter(*OrderMapFileName);
			OutputFile->SetIsTextFormat(true);
			OrderMap.WriteOpenOrder(OutputFile);
			OutputFile->Close();
			delete OutputFile;
		}

		return true;
	}

	if (FParse::Param(CmdLine, TEXT("AuditFiles")))
	{
		if (NonOptionArguments.Num() != 1)
		{
			UE_LOGF(LogPakFile, Error, "Incorrect arguments. Expected: -AuditFiles <PakFolder> -CSV=<OutputPath> [-OnlyDeleted] [-Order=<OrderingFile>] [-SortByOrdering]");
			return false;
		}
		
		FString PakFilenames = *NonOptionArguments[0];
		FPaths::MakeStandardFilename(PakFilenames);
		
		FString CSVFilename;
		FParse::Value( CmdLine, TEXT("CSV="), CSVFilename );

		bool bOnlyDeleted = FParse::Param( CmdLine, TEXT("OnlyDeleted") );
		bool bSortByOrdering = FParse::Param(CmdLine, TEXT("SortByOrdering"));

		FPakOrderMap OrderMap;
		FString GameOpenOrderStr;
		if (FParse::Value(CmdLine, TEXT("-order="), GameOpenOrderStr, false))
		{
			TArray<FString> GameOpenOrderFiles;
			GameOpenOrderStr.ParseIntoArray(GameOpenOrderFiles, TEXT(","), true);
			bool bMergeOrder = false;
			for (const FString& GameOpenOrder : GameOpenOrderFiles)
			{
				if (!OrderMap.ProcessOrderFile(*GameOpenOrder, false, bMergeOrder))
				{
					return false;
				}
				bMergeOrder = true;
			}
		}
		FString SecondOrderStr;
		if (FParse::Value(CmdLine, TEXT("-secondaryOrder="), SecondOrderStr, false))
		{
			TArray<FString> SecondOrderFiles;
			SecondOrderStr.ParseIntoArray(SecondOrderFiles, TEXT(","), true);
			for (const FString& SecondOpenOrder : SecondOrderFiles)
			{
				//We always merge the secondary order and keep the bSecondaryOrderFile for MaxPrimaryOrderIndex logic
				if (!OrderMap.ProcessOrderFile(*SecondOpenOrder, true, true))
				{
					return false;
				}
			}
		}

		return AuditPakFiles(*PakFilenames, bOnlyDeleted, CSVFilename, OrderMap, bSortByOrdering );
	}
	
	if (FParse::Param(CmdLine, TEXT("WhatsAtOffset")))
	{
		if (NonOptionArguments.Num() < 2)
		{
			UE_LOGF(LogPakFile, Error, "Incorrect arguments. Expected: -WhatsAtOffset <PakFile> [Offset...]");
			return false;
		}
		
		FString PakFilename = GetPakPath(*NonOptionArguments[0], false);

		TArray<int64> Offsets;
		for( int ArgI = 1; ArgI < NonOptionArguments.Num(); ArgI++ )
		{
			if( FCString::IsNumeric(*NonOptionArguments[ArgI]) )
			{
				Offsets.Add( FCString::Strtoi64( *NonOptionArguments[ArgI], nullptr, 10 ) );
			}
		}

		return ListFilesAtOffset( *PakFilename, Offsets );
	}

	if (FParse::Param(CmdLine, TEXT("CalcCompressionBlockCRCs")))
	{
		if (NonOptionArguments.Num() < 2)
		{
			UE_LOGF(LogPakFile, Error, "Incorrect arguments. Expected -CalcCompressionBlockCRCs <PakFile> [Offset...] " );
			return false;
		}

		FString PakFilename = GetPakPath(*NonOptionArguments[0], false);

		TArray<int64> Offsets;
		for( int ArgI = 1; ArgI < NonOptionArguments.Num(); ArgI++ )
		{
			if( FCString::IsNumeric(*NonOptionArguments[ArgI]) )
			{
				Offsets.Add( FCString::Strtoi64( *NonOptionArguments[ArgI], nullptr, 10 ) );
			}
		}

		return ShowCompressionBlockCRCs( *PakFilename, Offsets, KeyChain );
	}
	
	if (FParse::Param(CmdLine, TEXT("GeneratePIXMappingFile")))
	{
		if (NonOptionArguments.Num() != 1)
		{
			UE_LOGF(LogPakFile, Error, "Incorrect arguments. Expected: -GeneratePIXMappingFile <PakFile> [-OutputPath=<OutputPath>]");
			return false;
		}

		TArray<FString> PakFileList;
		const FString& PakFolderName = NonOptionArguments[0];
		if (FPaths::DirectoryExists(PakFolderName))
		{
			TArray<FString> PakFilesInFolder;
			IFileManager::Get().FindFiles(PakFilesInFolder, *PakFolderName, TEXT(".pak"));
			for (const FString& PakFile : PakFilesInFolder)
			{
				FString FullPakFileName = PakFolderName / PakFile;
				FullPakFileName.ReplaceInline(TEXT("/"), TEXT("\\"));
				PakFileList.AddUnique(GetPakPath(*FullPakFileName, false));
			}
		}

		FString OutputPath;
		FParse::Value(CmdLine, TEXT("OutputPath="), OutputPath);
		return GeneratePIXMappingFile(PakFileList, OutputPath);
	}

	if (FParse::Param(CmdLine, TEXT("Repack")))
	{
		if (NonOptionArguments.Num() != 1)
		{
			UE_LOGF(LogPakFile, Error, "Incorrect arguments. Expected: -Repack <PakFile> [-Output=<PakFile>]");
			return false;
		}

		TArray<FPakInputPair> Entries;
		FPakCommandLineParameters CmdLineParameters;
		ProcessCommandLine(CmdLine, NonOptionArguments, Entries, CmdLineParameters);

		// Find all the input pak files
		FString InputDir = FPaths::GetPath(*NonOptionArguments[0]);

		TArray<FString> InputPakFiles;
		if (FPaths::FileExists(NonOptionArguments[0]))
		{
			InputPakFiles.Add(NonOptionArguments[0]);
		}
		else
		{
			IFileManager::Get().FindFiles(InputPakFiles, *InputDir, *FPaths::GetCleanFilename(*NonOptionArguments[0]));

			for (int Idx = 0; Idx < InputPakFiles.Num(); Idx++)
			{
				InputPakFiles[Idx] = InputDir / InputPakFiles[Idx];
			}
		}

		if (InputPakFiles.Num() == 0)
		{
			UE_LOGF(LogPakFile, Error, "No files found matching '%ls'", *NonOptionArguments[0]);
			return false;
		}

		// Find all the output paths
		TArray<FString> OutputPakFiles;

		FString OutputPath;
		if (!FParse::Value(CmdLine, TEXT("Output="), OutputPath, false))
		{
			for (const FString& InputPakFile : InputPakFiles)
			{
				OutputPakFiles.Add(InputPakFile);
			}
		}
		else if (IFileManager::Get().DirectoryExists(*OutputPath))
		{
			for (const FString& InputPakFile : InputPakFiles)
			{
				OutputPakFiles.Add(FPaths::Combine(OutputPath, FPaths::GetCleanFilename(InputPakFile)));
			}
		}
		else
		{
			for (const FString& InputPakFile : InputPakFiles)
			{
				OutputPakFiles.Add(OutputPath);
			}
		}

		bool bExcludeDeleted = FParse::Param(CmdLine, TEXT("ExcludeDeleted"));

		// Repack them all
		for (int Idx = 0; Idx < InputPakFiles.Num(); Idx++)
		{
			UE_LOGF(LogPakFile, Display, "Repacking %ls into %ls", *InputPakFiles[Idx], *OutputPakFiles[Idx]);
			if (!Repack(InputPakFiles[Idx], OutputPakFiles[Idx], CmdLineParameters, KeyChain, !bExcludeDeleted))
			{
				return false;
			}
		}

		return true;
	}

	TArray<FString> CreatePakCommandsList;
	{
		FString CreateMultipleCommandListPath;
		if (FParse::Value(CmdLine, TEXT("-CreateMultiple="), CreateMultipleCommandListPath))
		{
			if (!FFileHelper::LoadFileToStringArray(CreatePakCommandsList, *CreateMultipleCommandListPath))
			{
				UE_LOGF(LogPakFile, Error, "Failed to read command list file '%ls'.", *CreateMultipleCommandListPath);
				return false;
			}
		}
	}
	if(NonOptionArguments.Num() > 0 || !CreatePakCommandsList.IsEmpty())
	{
		if (CreatePakCommandsList.IsEmpty())
		{
			CreatePakCommandsList.Add(CmdLine);
		}

		CheckAndReallocThreadPool();

		FPakCommandLineParameters CommonCmdLineParameters;
		ProcessCommonCommandLine(CmdLine, CommonCmdLineParameters);

		FPakWriterContext PakWriterContext;
		if (!PakWriterContext.Initialize(CommonCmdLineParameters))
		{
			return false;
		}

		FPakOrderMap OrderMap;
		FString GameOpenOrderStr;
		if (FParse::Value(CmdLine, TEXT("-order="), GameOpenOrderStr, false))
		{
			TArray<FString> GameOpenOrderFiles;
			GameOpenOrderStr.ParseIntoArray(GameOpenOrderFiles, TEXT(","), true);
			bool bMergeOrder = false;
			for (const FString& GameOpenOrder : GameOpenOrderFiles)
			{
				if (!OrderMap.ProcessOrderFile(*GameOpenOrder, false, bMergeOrder))
				{
					return false;
				}
				bMergeOrder = true;
			}
		}

		FString SecondOrderStr;
		if (FParse::Value(CmdLine, TEXT("-secondaryOrder="), SecondOrderStr, false))
		{
			TArray<FString> SecondOrderFiles;
			SecondOrderStr.ParseIntoArray(SecondOrderFiles, TEXT(","), true);
			for (const FString& SecondOpenOrder : SecondOrderFiles)
			{
				//We always merge the secondary order and keep the bSecondaryOrderFile for MaxPrimaryOrderIndex logic
				if (!OrderMap.ProcessOrderFile(*SecondOpenOrder, true, true))
				{
					return false;
				}
			}
		}

		// Check command line for the "patchcryptokeys" param, which will tell us where to look for the encryption keys that
		// we need to access the patch reference data
		FString PatchReferenceCryptoKeysFilename;
		FKeyChain PatchKeyChain;

		if (FParse::Value(FCommandLine::Get(), TEXT("PatchCryptoKeys="), PatchReferenceCryptoKeysFilename))
		{
			KeyChainUtilities::LoadKeyChainFromFile(PatchReferenceCryptoKeysFilename, PatchKeyChain);
			KeyChainUtilities::ApplyEncryptionKeys(PatchKeyChain);
		}
		else
		{
			PatchKeyChain = KeyChain;
		}

		KeyChainUtilities::ApplyEncryptionKeys(KeyChain);

		TArray<FString> TempOutputDirectoriesToDelete;
		bool bResult = true;
		for (const FString& CreatePakCommand : CreatePakCommandsList)
		{
			TArray<FString> NonOptionArgumentsForPakFile;
			for (const TCHAR* CmdLineEnd = *CreatePakCommand; *CmdLineEnd != 0;)
			{
				FString Argument = FParse::Token(CmdLineEnd, false);
				if (Argument.Len() > 0 && !Argument.StartsWith(TEXT("-")))
				{
					NonOptionArgumentsForPakFile.Add(Argument);
				}
			}

			// since this is for creation, we pass true to make it not look in LaunchDir
			FString PakFilename = GetPakPath(*NonOptionArgumentsForPakFile[0], true);
			
			// List of all items to add to pak file
			TArray<FPakInputPair> Entries;			
			FPakCommandLineParameters CmdLineParameters = CommonCmdLineParameters;
			ProcessPakFileSpecificCommandLine(*CreatePakCommand, NonOptionArgumentsForPakFile, Entries, CmdLineParameters);
			
			if (CmdLineParameters.bRequiresRehydration)
			{
				if (!InitializeVirtualizationSystem())
				{
					bResult = false;
					break;
				}
			}

			FKeyChain KeyChainForPakFile = KeyChain;
			FString EncryptionKeyOverrideGuidString;
			FGuid EncryptionKeyOverrideGuid;
			if (FParse::Value(*CreatePakCommand, TEXT("EncryptionKeyOverrideGuid="), EncryptionKeyOverrideGuidString))
			{
				FGuid::Parse(EncryptionKeyOverrideGuidString, EncryptionKeyOverrideGuid);
			}
			KeyChainForPakFile.SetPrincipalEncryptionKey( KeyChainForPakFile.GetEncryptionKeys().Find(EncryptionKeyOverrideGuid) );
		
			int32 LowestSourcePakVersion = 0;
			TMap<FString, FFileInfo> SourceFileHashes;

			if (CmdLineParameters.GeneratePatch)
			{
				FString TempOutputPath;
				if (!FParse::Value(*CreatePakCommand, TEXT("TempFiles="), TempOutputPath))
				{
					TempOutputPath = FPaths::GetPath(PakFilename) / FString(TEXT("TempFiles"));
				}

				IFileManager::Get().DeleteDirectory(*TempOutputPath);

				UE_LOGF(LogPakFile, Display, "Generating patch from %ls.", *CmdLineParameters.SourcePatchPakFilename);

				TArray<FGuid> UsedEncryptionKeys;
				if (!GenerateHashesFromPak(*CmdLineParameters.SourcePatchPakFilename, *PakFilename, SourceFileHashes, true, PatchKeyChain, /*Out*/LowestSourcePakVersion, &UsedEncryptionKeys))
				{
					UE_LOGF(LogPakFile, Warning, "Unable to generate hashes from pak file %ls", *CmdLineParameters.SourcePatchPakFilename);
					if (ExtractFilesFromPak(*CmdLineParameters.SourcePatchPakFilename, SourceFileHashes, *TempOutputPath, true, PatchKeyChain, nullptr, nullptr, nullptr, nullptr, &UsedEncryptionKeys) == false)
					{
						UE_LOGF(LogPakFile, Warning, "Unable to extract files from source pak file for patch");
					}
					else
					{
						TempOutputDirectoriesToDelete.Add(TempOutputPath);
						CmdLineParameters.SourcePatchDiffDirectory = TempOutputPath;
					}
				}

				if (UsedEncryptionKeys.Num() == 1)
				{
					if (UsedEncryptionKeys[0].IsValid())
					{
						UE_LOGF(LogPakFile, Display, "Found encryption key %ls in pak file %ls will use to encrypt patch", *((UsedEncryptionKeys[0]).ToString()), *CmdLineParameters.SourcePatchPakFilename);
						KeyChainForPakFile.SetPrincipalEncryptionKey(KeyChainForPakFile.GetEncryptionKeys().Find(UsedEncryptionKeys[0]));
					}
				}
			}

			// Start collecting files
			TArray<FPakInputPair> FilesToAdd;
			CollectFilesToAdd(FilesToAdd, Entries, OrderMap, CmdLineParameters);

			if (FParse::Param(CmdLine, TEXT("writepakchunkorder")))
			{
				FString PakOrderFilename = FPaths::GetPath(PakFilename) + FPaths::GetBaseFilename(PakFilename) + FString(TEXT("-order")) + FString(TEXT(".txt"));
				FArchive* PakOrderListArchive = IFileManager::Get().CreateFileWriter(*PakOrderFilename);
				PakOrderListArchive->SetIsTextFormat(true);

				for (int32 I = 0; I < FilesToAdd.Num(); ++I)
				{
					const FPakInputPair& Entry = FilesToAdd[I];
					FString Line = FString::Printf(TEXT("%s %lld"), *Entry.Source, Entry.SuggestedOrder);
					PakOrderListArchive->Logf(TEXT("%s"), *Line);
				}
				PakOrderListArchive->Close();
				delete PakOrderListArchive;
			}

			if (CmdLineParameters.GeneratePatch)
			{
				// We need to get a list of files that were in the previous patch('s) Pak, but NOT in FilesToAdd
				TArray<FPakInputPair> DeleteRecords = GetNewDeleteRecords(FilesToAdd, SourceFileHashes);

				//if the patch is built using old source pak files, we need to handle the special case where a file has been moved between chunks but no delete record was created (this would cause a rogue delete record to be created in the latest pak), and also a case where the file was moved between chunks and back again without being changed (this would cause the file to not be included in this chunk because the file would be considered unchanged)
				if (LowestSourcePakVersion < FPakInfo::PakFile_Version_DeleteRecords)
				{
					int32 CurrentPatchChunkIndex = GetPakChunkIndexFromFilename(PakFilename);

					UE_LOGF(LogPakFile, Display, "Some patch source paks were generated with an earlier version of UnrealPak that didn't support delete records. checking for historic assets that have moved between chunks to avoid creating invalid delete records");
					FString SourcePakFolder = FPaths::GetPath(CmdLineParameters.SourcePatchPakFilename);

					//remove invalid items from DeleteRecords and set 'bForceInclude' on some SourceFileHashes
					ProcessLegacyFileMoves(DeleteRecords, SourceFileHashes, SourcePakFolder, FilesToAdd, CurrentPatchChunkIndex);
				}
				FilesToAdd.Append(DeleteRecords);

				// if we are generating a patch here we remove files which are already shipped...
				RemoveIdenticalFiles(FilesToAdd, CmdLineParameters.SourcePatchDiffDirectory, SourceFileHashes, CmdLineParameters.SeekOptParams, CmdLineParameters.ChangedFilesOutputFilename);
			}

			if (!PakWriterContext.AddPakFile(*PakFilename, FilesToAdd, KeyChainForPakFile))
			{
				bResult = false;
				break;
			}
		}

		bResult &= PakWriterContext.Flush();

		for (const FString& TempOutputDirectory : TempOutputDirectoriesToDelete)
		{
			// delete the temporary directory
			IFileManager::Get().DeleteDirectory(*TempOutputDirectory, false, true);
		}

#if USE_DDC_FOR_COMPRESSED_FILES
		GetDerivedDataCacheRef().WaitForQuiescence(true);
#endif

		return bResult;
	}

	UE_LOGF(LogPakFile, Error, "No pak file name specified. Usage:");
	UE_LOGF(LogPakFile, Error, "  UnrealPak <PakFilename> -Test");
	UE_LOGF(LogPakFile, Error, "  UnrealPak <PakFilename> -Verify");
	UE_LOGF(LogPakFile, Error, "  UnrealPak <PakFilename> -Info");
	UE_LOGF(LogPakFile, Error, "  UnrealPak <PakFilename> -List [-ExcludeDeleted]");
	UE_LOGF(LogPakFile, Error, "  UnrealPak <PakFilename> <GameUProjectName> <GameFolderName> -ExportDependencies=<OutputFileBase> -NoAssetRegistryCache -ForceDependsGathering");
	UE_LOGF(LogPakFile, Error, "  UnrealPak <PakFilename> -Extract <ExtractDir> [-Filter=<filename>]");
	UE_LOGF(LogPakFile, Error, "  UnrealPak <PakFilename> -Create=<ResponseFile> [Options]");
	UE_LOGF(LogPakFile, Error, "  UnrealPak <PakFilename> -Dest=<MountPoint>");
	UE_LOGF(LogPakFile, Error, "  UnrealPak <PakFilename> -Repack [-Output=Path] [-ExcludeDeleted] [Options]");
	UE_LOGF(LogPakFile, Error, "  UnrealPak <PakFilename1> <PakFilename2> -diff");
	UE_LOGF(LogPakFile, Error, "  UnrealPak <PakFolder> -AuditFiles [-OnlyDeleted] [-CSV=<filename>] [-order=<OrderingFile>] [-SortByOrdering]");
	UE_LOGF(LogPakFile, Error, "  UnrealPak <PakFilename> -WhatsAtOffset [offset1] [offset2] [offset3] [...]");
	UE_LOGF(LogPakFile, Error, "  UnrealPak <PakFolder> -GeneratePIXMappingFile -OutputPath=<Path>");
	UE_LOGF(LogPakFile, Error, "  Options:");
	UE_LOGF(LogPakFile, Error, "    -blocksize=<BlockSize>");
	UE_LOGF(LogPakFile, Error, "    -bitwindow=<BitWindow>");
	UE_LOGF(LogPakFile, Error, "    -compress");
	UE_LOGF(LogPakFile, Error, "    -encrypt");
	UE_LOGF(LogPakFile, Error, "    -order=<OrderingFile>");
	UE_LOGF(LogPakFile, Error, "    -diff (requires 2 filenames first)");
	UE_LOGF(LogPakFile, Error, "    -enginedir (specify engine dir for when using ini encryption configs)");
	UE_LOGF(LogPakFile, Error, "    -projectdir (specify project dir for when using ini encryption configs)");
	UE_LOGF(LogPakFile, Error, "    -cryptokeys=<cryptofile> (For example Crypto.json in cooked data)");
	UE_LOGF(LogPakFile, Error, "    -encryptionini (specify ini base name to gather encryption settings from)");
	UE_LOGF(LogPakFile, Error, "    -extracttomountpoint (Extract to mount point path of pak file)");
	UE_LOGF(LogPakFile, Error, "    -encryptindex (encrypt the pak file index, making it unusable in unrealpak without supplying the key)");
	UE_LOGF(LogPakFile, Error, "    -compressionformat[s]=<Format[,format2,...]> (set the format(s) to compress with, falling back on failures)");
	UE_LOGF(LogPakFile, Error, "    -encryptionkeyoverrideguid (override the encryption key guid used for encrypting data in this pak file)");
	UE_LOGF(LogPakFile, Error, "    -sign (generate a signature (.sig) file alongside the pak)");
	UE_LOGF(LogPakFile, Error, "    -fallbackOrderForNonUassetFiles (if order is not specified for ubulk/uexp files, figure out implicit order based on the uasset order. Generally applies only to the cooker order)");

	return false;
}
