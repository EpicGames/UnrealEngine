// Copyright Epic Games, Inc. All Rights Reserved.

#include "SignedArchiveWriter.h"
#include "IPlatformFilePak.h"
#include "Misc/SecureHash.h"
#include "HAL/FileManager.h"

FSignedArchiveWriter::FSignedArchiveWriter(FArchive& InPak, const FString& InPakFilename, const FRSAKeyHandle InSigningKey)
: FArchive(InPak)
, BufferArchive(Buffer)
, PakWriter(InPak)
, PakSignaturesFilename(FPaths::ChangeExtension(InPakFilename, TEXT("sig")))
, SizeOnDisk(0)
, PakSize(0)
, SigningKey(InSigningKey)
{
	check(IsSaving());
	Buffer.Reserve(FPakInfo::MaxChunkDataSize);
}

FSignedArchiveWriter::~FSignedArchiveWriter()
{
	if (BufferArchive.Tell() > 0)
	{
		SerializeBufferAndSign();
	}
	delete &PakWriter;
}

void FSignedArchiveWriter::SerializeBufferAndSign()
{
	// Compute a hash for this buffer data
	ChunkHashes.Add(ComputePakChunkHash(&Buffer[0], Buffer.Num()));

	// Flush the buffer
	PakWriter.Serialize(&Buffer[0], Buffer.Num());
	BufferArchive.Seek(0);
	Buffer.Empty(FPakInfo::MaxChunkDataSize);
}

bool FSignedArchiveWriter::Close()
{
	if (BufferArchive.Tell() > 0)
	{
		SerializeBufferAndSign();
	}

	FArchive* SignatureWriter = IFileManager::Get().CreateFileWriter(*PakSignaturesFilename);
	FPakSignatureFile SignatureFile;
	SignatureFile.SetChunkHashesAndSign(ChunkHashes, SignatureData, SigningKey);
	SignatureFile.Serialize(*SignatureWriter);
	delete SignatureWriter;

	return FArchive::Close();
}

void FSignedArchiveWriter::Serialize(void* Data, int64 Length)
{
	// Serialize data to a buffer. When the max buffer size is reached, the buffer is signed and
	// serialized to disk with its signature
	uint8* DataToWrite = (uint8*)Data;
	int64 RemainingSize = Length;
	while (RemainingSize > 0)
	{
		int64 BufferPos = BufferArchive.Tell();
		int64 SizeToWrite = RemainingSize;
		if (BufferPos + SizeToWrite > FPakInfo::MaxChunkDataSize)
		{
			SizeToWrite = FPakInfo::MaxChunkDataSize - BufferPos;
		}

		BufferArchive.Serialize(DataToWrite, SizeToWrite);
		if (BufferArchive.Tell() == FPakInfo::MaxChunkDataSize)
		{
			SerializeBufferAndSign();
		}
			
		SizeOnDisk += SizeToWrite;
		PakSize += SizeToWrite;

		RemainingSize -= SizeToWrite;
		DataToWrite += SizeToWrite;
	}
}

int64 FSignedArchiveWriter::Tell()
{
	return PakSize;
}

int64 FSignedArchiveWriter::TotalSize()
{
	return PakSize;
}

void FSignedArchiveWriter::Seek(int64 InPos)
{
	UE_LOGF(LogPakFile, Fatal, "Seek is not supported in FSignedArchiveWriter.");
}