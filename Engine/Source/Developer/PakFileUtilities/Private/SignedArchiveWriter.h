// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryWriter.h"
#include "Math/BigInt.h"
#include "IPlatformFilePak.h"

/**
 * Wrapper for writing and signing an archive
 */
class FSignedArchiveWriter : public FArchive
{
	/** Buffer to sign */
	TArray<uint8> Buffer;
	/** Buffer writer */
	FMemoryWriter BufferArchive;
	/** The actual pak archive */
	FArchive& PakWriter;
	/** The filename of the signature file that accompanies the pak */
	FString PakSignaturesFilename;
	/** Size of the archive on disk (including signatures) */
	int64 SizeOnDisk;
	/** Data size (excluding signatures) */
	int64 PakSize;
	/** Signing key */
	const FRSAKeyHandle SigningKey;
	/** Hashes */
	TArray<TPakChunkHash> ChunkHashes;
	/** Signature data */
	TArray<uint8> SignatureData;

	/** 
	 * Serializes and signs a buffer
	 */
	void SerializeBufferAndSign();

public:

	FSignedArchiveWriter(FArchive& InPak, const FString& InPakFilename, const FRSAKeyHandle InSigningKey);
	virtual ~FSignedArchiveWriter();

	// FArchive interface
	virtual bool Close() override;
	virtual void Serialize(void* Data, int64 Length) override;
	virtual int64 Tell() override;
	virtual int64 TotalSize() override;
	virtual void Seek(int64 InPos) override;

	void SetSignatureData(TArray<uint8>& InSignatureData)
	{
		SignatureData = MoveTemp(InSignatureData);
	}
};
