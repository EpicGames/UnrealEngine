// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/KeyChainUtilities.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS 

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FKeyChainTest, "System.Core.Misc.KeyChain", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

/**
 * KeyChain tests - implemented in this module because of incorrect dependencies in KeyChainUtilities.h.
 */
bool FKeyChainTest::RunTest(const FString& Parameters)
{
	const FGuid DefaultGuid;
	uint32 SigningKeyDummy = 42;
	const FRSAKeyHandle DefaultSigningKey = reinterpret_cast<const FRSAKeyHandle>(&SigningKeyDummy);

	// Default construct
	{
		FKeyChain Chain;
		TestTrue(TEXT("Default construct - principal key is invalid"), Chain.GetPrincipalEncryptionKey() == nullptr);
		TestTrue(TEXT("Default construct - signing key is invalid"), Chain.GetSigningKey() == nullptr);
		TestTrue(TEXT("Default construct - encryption keys are empty"), Chain.GetEncryptionKeys().Num() == 0);
	}

	// Copy construct
	{
		FKeyChain Chain;
		Chain.GetEncryptionKeys().Add(DefaultGuid, FNamedAESKey { TEXT("Default"), DefaultGuid, FAES::FAESKey() });
		Chain.SetPrincipalEncryptionKey(Chain.GetEncryptionKeys().Find(DefaultGuid));
		Chain.SetSigningKey(DefaultSigningKey);

		FKeyChain Copy(Chain);

		TestTrue(TEXT("Copy construct - with valid principal key does NOT copy pointer"),
			Copy.GetPrincipalEncryptionKey() != Chain.GetPrincipalEncryptionKey());
		TestTrue(TEXT("Copy construct - principal key name matches"),
			Copy.GetPrincipalEncryptionKey()->Name == Chain.GetPrincipalEncryptionKey()->Name);
		TestTrue(TEXT("Copy construct - principal key GUID name matches"),
			Copy.GetPrincipalEncryptionKey()->Guid == Chain.GetPrincipalEncryptionKey()->Guid);
		TestTrue(TEXT("Copy construct - with valid principal key sets principal key"),
			Copy.GetPrincipalEncryptionKey() == Copy.GetEncryptionKeys().Find(DefaultGuid));
		TestTrue(TEXT("Copy construct - signing key matches"),
			Copy.GetSigningKey() == Chain.GetSigningKey());
	}

	// Copy assign
	{
		{
			FKeyChain Chain;
			Chain.GetEncryptionKeys().Add(DefaultGuid, FNamedAESKey{TEXT("Default"), DefaultGuid, FAES::FAESKey()});
			Chain.SetPrincipalEncryptionKey(Chain.GetEncryptionKeys().Find(DefaultGuid));
			Chain.SetSigningKey(DefaultSigningKey);

			FKeyChain Copy;
			Copy = Chain;
			
			TestTrue(TEXT("Copy assign - with valid principal key does NOT copy pointer"),
				Copy.GetPrincipalEncryptionKey() != Chain.GetPrincipalEncryptionKey());
			TestTrue(TEXT("Copy assign - principal key name matches"),
				Copy.GetPrincipalEncryptionKey()->Name == Chain.GetPrincipalEncryptionKey()->Name);
			TestTrue(TEXT("Copy assign - principal key GUID name matches"),
				Copy.GetPrincipalEncryptionKey()->Guid == Chain.GetPrincipalEncryptionKey()->Guid);
			TestTrue(TEXT("Copy assign - with valid principal key sets principal key"),
				Copy.GetPrincipalEncryptionKey() == Copy.GetEncryptionKeys().Find(DefaultGuid));
			TestTrue(TEXT("Copy assign - signing key matches"),
				Copy.GetSigningKey() == Chain.GetSigningKey());
		}

		{
			FKeyChain Copy;
			Copy.GetEncryptionKeys().Add(DefaultGuid, FNamedAESKey { TEXT("Default"), DefaultGuid, FAES::FAESKey() });
			Copy.SetPrincipalEncryptionKey(Copy.GetEncryptionKeys().Find(DefaultGuid));
			Copy.SetSigningKey(DefaultSigningKey);

			FKeyChain Chain;
			Copy = Chain;
			
			TestTrue(TEXT("Copy assign - empty instance, clears principal key"),
				Copy.GetPrincipalEncryptionKey() == nullptr);
			TestTrue(TEXT("Copy assign - empty instance, clears encryption keys"),
				Copy.GetEncryptionKeys().Num() == 0);
			TestTrue(TEXT("Copy assign - signing key is invalid"),
				Copy.GetSigningKey() == nullptr);
		}
	}

	// Move construct
	{
		FKeyChain Moved;
		Moved.GetEncryptionKeys().Add(DefaultGuid, FNamedAESKey { TEXT("Default"), DefaultGuid, FAES::FAESKey() });
		Moved.SetPrincipalEncryptionKey(Moved.GetEncryptionKeys().Find(DefaultGuid));
		Moved.SetSigningKey(DefaultSigningKey);

		FKeyChain Chain(MoveTemp(Moved));

		TestTrue(TEXT("Move construct - with valid principal key sets principal key"),
			Chain.GetPrincipalEncryptionKey() == Chain.GetEncryptionKeys().Find(DefaultGuid));
		TestTrue(TEXT("Move construct - with valid principal key sets signing key"),
			Chain.GetSigningKey() == DefaultSigningKey); 
		TestTrue(TEXT("Move construct - invalidates moved instance"),
			Moved.GetPrincipalEncryptionKey() == nullptr);
		TestTrue(TEXT("Move construct - invalidates moved instance"),
			Moved.GetSigningKey() == InvalidRSAKeyHandle);
		TestTrue(TEXT("Move construct - invalidates moved instance"),
			Moved.GetEncryptionKeys().Num() == 0);
	}

	// Move assign
	{
		{
			FKeyChain Moved;
			Moved.GetEncryptionKeys().Add(DefaultGuid, FNamedAESKey{TEXT("Default"), DefaultGuid, FAES::FAESKey()});
			Moved.SetPrincipalEncryptionKey(Moved.GetEncryptionKeys().Find(DefaultGuid));
			Moved.SetSigningKey(DefaultSigningKey);

			FKeyChain Chain;
			Chain = MoveTemp(Moved);

			TestTrue(TEXT("Move construct - with valid principal key sets principal key"),
				Chain.GetPrincipalEncryptionKey() == Chain.GetEncryptionKeys().Find(DefaultGuid));
			TestTrue(TEXT("Move construct - with valid principal key sets signing key"),
				Chain.GetSigningKey() == DefaultSigningKey);
			TestTrue(TEXT("Move construct - invalidates moved instance"),
				Moved.GetPrincipalEncryptionKey() == nullptr);
			TestTrue(TEXT("Move construct - invalidates moved instance"),
				Moved.GetSigningKey() == InvalidRSAKeyHandle);
			TestTrue(TEXT("Move construct - invalidates moved instance"),
				Moved.GetEncryptionKeys().Num() == 0);
		}
		
		{
			FKeyChain Copy;
			Copy.GetEncryptionKeys().Add(DefaultGuid, FNamedAESKey{TEXT("Default"), DefaultGuid, FAES::FAESKey()});
			Copy.SetPrincipalEncryptionKey(Copy.GetEncryptionKeys().Find(DefaultGuid));

			Copy = FKeyChain();

			TestTrue(TEXT("Move assign - empty instance, clears principal key"),
				Copy.GetPrincipalEncryptionKey() == nullptr);
			TestTrue(TEXT("Move assign - empty instance, clears encryption keys"),
				Copy.GetEncryptionKeys().Num() == 0);
			TestTrue(TEXT("Move assign - invalidates signing key"),
				Copy.GetSigningKey() == InvalidRSAKeyHandle);
		}
	}

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS
