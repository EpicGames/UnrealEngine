// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/PlatformCrt.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/Optional.h"
#include "Misc/IEngineCrypto.h"
#include "Templates/Tuple.h"

class FArchive;
struct FFileRegion;
struct FKeyChain;
struct FPakEntryPair;
struct FPakInfo;

/**
* Defines the order mapping for files within a pak.
* When read from the files present in the pak, Indexes will be [0,NumFiles).  This is important for detecting gaps in the order between adjacent files in a patch .pak.
* For new files being added into the pak, the values can be arbitrary, and will be usable only for relative order in an output list.
* Due to the arbitrary values for new files, the FPakOrderMap can contain files with duplicate Order values.
*/
class FPakOrderMap
{
public:
	FPakOrderMap()
		: MaxPrimaryOrderIndex(MAX_uint64)
		, MaxIndex(0)
	{}

	void Empty()
	{
		OrderMap.Empty();
		MaxPrimaryOrderIndex = MAX_uint64;
		MaxIndex = 0;
	}

	int32 Num() const
	{
		return OrderMap.Num();
	}

	/** Add the given filename with the given Sorting Index */
	void Add(const FString& Filename, uint64 Index)
	{
		OrderMap.Add(Filename, Index);
		MaxIndex = FMath::Max(MaxIndex, Index);
	}

	/**
	* Add the given filename with the given Offset interpreted as Offset in bytes in the Pak File.  This version of Add is only useful when all Adds are done by offset, and are converted
	* into Sorting Indexes at the end by a call to ConvertOffsetsToOrder
	*/
	void AddOffset(const FString& Filename, uint64 Offset)
	{
		OrderMap.Add(Filename, Offset);
		MaxIndex = FMath::Max(MaxIndex, Offset);
	}

	/** Remaps all the current values in the OrderMap onto [0, NumEntries).  Useful to convert from Offset in Pak file bytes into an Index sorted by Offset */
	void ConvertOffsetsToOrder()
	{
		TArray<TPair<FString, uint64>> FilenameAndOffsets;
		for (auto& FilenameAndOffset : OrderMap)
		{
			FilenameAndOffsets.Add(FilenameAndOffset);
		}
		FilenameAndOffsets.Sort([](const TPair<FString, uint64>& A, const TPair<FString, uint64>& B)
		{
			return A.Value < B.Value;
		});
		int64 Index = 0;
		for (auto& FilenameAndOffset : FilenameAndOffsets)
		{
			OrderMap[FilenameAndOffset.Key] = Index;
			++Index;
		}
		MaxIndex = Index - 1;
	}

	bool PAKFILEUTILITIES_API ProcessOrderFile(const TCHAR* ResponseFile, bool bSecondaryOrderFile = false, bool bMergeOrder = false, TOptional<uint64> InOffset = {});

	// Merge another order map into this one where the files are not already ordered by this map. Steals the strings and empties the other order map.
	void PAKFILEUTILITIES_API MergeOrderMap(FPakOrderMap&& Other);

	uint64 PAKFILEUTILITIES_API GetFileOrder(const FString& Path, bool bAllowUexpUBulkFallback, bool* OutIsPrimary=nullptr) const;

	void PAKFILEUTILITIES_API WriteOpenOrder(FArchive* Ar);

	uint64 GetMaxIndex() { return MaxIndex; }

private:
	FString RemapLocalizationPathIfNeeded(const FString& PathLower, FString& OutRegion) const;

	TMap<FString, uint64> OrderMap;
	uint64 MaxPrimaryOrderIndex;
	uint64 MaxIndex;
};


PAKFILEUTILITIES_API bool ExecuteUnrealPak(const TCHAR* CmdLine);

/** Input and output data for WritePakFooter */
struct FPakFooterInfo
{
	FPakFooterInfo(const TCHAR* InFilename, const FString& InMountPoint, FPakInfo& InInfo, TArray<FPakEntryPair>& InIndex)
		: Filename(InFilename)
		, MountPoint(InMountPoint)
		, Info(InInfo)
		, Index(InIndex)
	{
	}
	void SetEncryptionInfo(const FKeyChain& InKeyChain, uint64* InTotalEncryptedDataSize)
	{
		KeyChain = &InKeyChain;
		TotalEncryptedDataSize = InTotalEncryptedDataSize;
	}
	void SetFileRegionInfo(bool bInFileRegions, TArray<FFileRegion>& InAllFileRegions)
	{
		bFileRegions = bInFileRegions;
		AllFileRegions = &InAllFileRegions;
	}

	const TCHAR* Filename;
	const FString& MountPoint;
	FPakInfo& Info;
	TArray<FPakEntryPair>& Index;

	const FKeyChain* KeyChain = nullptr;
	uint64* TotalEncryptedDataSize = nullptr;
	bool bFileRegions = false;
	TArray<FFileRegion>* AllFileRegions = nullptr;

	int64 PrimaryIndexSize = 0;
	int64 PathHashIndexSize = 0;
	int64 FullDirectoryIndexSize = 0;
};

/** Write the index and other data at the end of a pak file after all the entries */
PAKFILEUTILITIES_API void WritePakFooter(FArchive& PakHandle, FPakFooterInfo& FooterInfo);

/** Take an existing pak file and regenerate the signature file */
PAKFILEUTILITIES_API bool SignPakFile(const FString& InPakFilename, const FRSAKeyHandle InSigningKey);
